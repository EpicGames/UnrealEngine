// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class Analytics : ModuleRules
	{
		public Analytics(ReadOnlyTargetRules Target) : base(Target)
		{			
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
                    // ... add other public dependencies that you statically link with here ...
				}
				);
			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
		}
	}
}
