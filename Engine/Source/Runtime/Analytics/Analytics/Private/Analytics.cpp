// Copyright Epic Games, Inc. All Rights Reserved.

#include "Analytics.h"
#include "AnalyticsTracer.h"
#include "Misc/CommandLine.h"
#include "Interfaces/IAnalyticsProviderModule.h"

DEFINE_LOG_CATEGORY(LogAnalytics);

IMPLEMENT_MODULE( FAnalytics, Analytics );

FAnalytics::FAnalytics()
{
}

FAnalytics::~FAnalytics()
{
}


TSharedPtr<IAnalyticsProvider> FAnalytics::CreateAnalyticsProvider( const FName& ProviderModuleName, const FAnalyticsProviderConfigurationDelegate& GetConfigValue )
{
	// Check if we can successfully load the module.
	if (ProviderModuleName != NAME_None)
	{
		IAnalyticsProviderModule* Module = FModuleManager::Get().LoadModulePtr<IAnalyticsProviderModule>(ProviderModuleName);
		if (Module != NULL)
		{
			UE_LOGF(LogAnalytics, Log, "Creating configured Analytics provider %ls", *ProviderModuleName.ToString());
			return Module->CreateAnalyticsProvider(GetConfigValue);
		}
		else
		{
			UE_LOGF(LogAnalytics, Warning, "Failed to find Analytics provider named %ls.", *ProviderModuleName.ToString());
		}
	}
	else
	{
		UE_LOGF(LogAnalytics, Warning, "CreateAnalyticsProvider called with a module name of None.");
	}
	return NULL;
}

TSharedPtr<IAnalyticsTracer> FAnalytics::CreateAnalyticsTracer()
{
	return MakeShared<FAnalyticsTracer>();
}

FString FAnalytics::GetConfigValueFromIni( const FString& IniName, const FString& SectionName, const FString& KeyName, bool bIsRequired )
{
	FString Result;
	if (!GConfig->GetString(*SectionName, *KeyName, Result, IniName) && bIsRequired)
	{
		UE_LOGF(LogAnalytics, Verbose, "Analytics missing Key %ls from %ls[%ls].", *KeyName, *IniName, *SectionName);
	}
	return Result;
}

void FAnalytics::WriteConfigValueToIni(const FString& IniName, const FString& SectionName, const FString& KeyName, const FString& Value)
{
	GConfig->SetString(*SectionName, *KeyName, *Value, *IniName);
}


void FAnalytics::StartupModule()
{
}

void FAnalytics::ShutdownModule()
{
}

ANALYTICS_API EAnalyticsBuildType GetAnalyticsBuildType()
{
	// Detect release packaging
	// @todo - find some way to decide if we're really release mode
	bool bIsRelease = false;
#if UE_BUILD_SHIPPING 
	bIsRelease = true;
#endif
	if (bIsRelease) return EAnalyticsBuildType::Release;

	return FParse::Param(FCommandLine::Get(), TEXT("DEBUGANALYTICS"))
		? EAnalyticsBuildType::Debug
		: FParse::Param(FCommandLine::Get(), TEXT("TESTANALYTICS"))
			? EAnalyticsBuildType::Test
			: FParse::Param(FCommandLine::Get(), TEXT("RELEASEANALYTICS"))
				? EAnalyticsBuildType::Release
				: EAnalyticsBuildType::Development;
}

ANALYTICS_API const TCHAR* AnalyticsBuildTypeToString(EAnalyticsBuildType Type)
{
	switch (Type)
	{
	case EAnalyticsBuildType::Development:		return TEXT("Development");
	case EAnalyticsBuildType::Test:				return TEXT("Test");
	case EAnalyticsBuildType::Debug:			return TEXT("Debug");
	case EAnalyticsBuildType::Release:			return TEXT("Release");
	default:
		break;
	}

	return TEXT("Undefined");
}



