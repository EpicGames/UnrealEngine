// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnalyticsPropertyStore.h"
#include "AutoRTFM.h"
#include "Containers/StringConv.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformTime.h"
#include "Misc/SecureHash.h"
#include "Misc/ScopeExit.h"
#include "Misc/ScopeLock.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Templates/UnrealTemplate.h"

namespace AnalyticsPropertyStoreUtils
{
	/** The key/value storage format version defining the memory layout of the store. */
	static const uint32 StoreFormatVersion = 1;

	/** MD5 checksum is 16 bytes long. */
	static constexpr uint32 Md5DigestSize = 16;

	/** The store data header that should never change. */
	struct FAnalyticsStoreFileHeader
	{
		uint32 Version;
		uint32 DataSize;
		uint8 Checksum[Md5DigestSize];
	};

	/** Conditionally set a value in the store. */
	template<typename T, typename TGetter, typename TSetter, typename TCompare>
	IAnalyticsPropertyStore::EStatusCode SetValue(TGetter&& GetterFn, TSetter&& SetterFn, const T& ProposedValue, TCompare&& ConditionFn)
	{
		T ActualValue;
		IAnalyticsPropertyStore::EStatusCode Status = GetterFn(ActualValue);
		if (Status == IAnalyticsPropertyStore::EStatusCode::Success || Status == IAnalyticsPropertyStore::EStatusCode::NotFound)
		{
			// Apply the condition.
			if (ConditionFn(Status == IAnalyticsPropertyStore::EStatusCode::NotFound ? nullptr : &ActualValue, ProposedValue))
			{
				return SetterFn(ProposedValue);
			}
			else
			{
				return IAnalyticsPropertyStore::EStatusCode::Declined;
			}
		}
		return Status;
	}

	/** Conditionally update a value in the store. */
	template<typename T, typename TGetter, typename TSetter, typename TCompare>
	IAnalyticsPropertyStore::EStatusCode UpdateValue(TGetter&& GetterFn, TSetter&& SetterFn, TCompare&& UpdateFn)
	{
		T Value;
		IAnalyticsPropertyStore::EStatusCode Status = GetterFn(Value);
		if (Status == IAnalyticsPropertyStore::EStatusCode::Success)
		{
			// Call back and check if the value was updated
			if (UpdateFn(Value))
			{
				return SetterFn(Value);
			}
			else
			{
				return IAnalyticsPropertyStore::EStatusCode::Declined;
			}
		}
		return Status;
	}
} // namespace AnalyticsPropertyStoreUtils


FAnalyticsPropertyStore::FAnalyticsPropertyStore()
	: StorageWriter(StorageBuf)
	, StorageReader(StorageBuf)
{
}

FAnalyticsPropertyStore::~FAnalyticsPropertyStore()
{
	if (AutoRTFM::IsClosed())
	{
		// The FAnalyticsPropertyStore must not be destroyed from within a transaction, 
		// since any queued-up flush tasks won't be run. 
		AutoRTFM::PopAllOnCommitHandlers(this);
		ensure(!"Destroying FAnalyticsPropertyStore from within a transaction!");
	}
	else
	{
		// Ensure any running async flush has completed.
		UE::TScopeLock Lock(StoreLock);
		if (FlushTask.IsValid())
		{
			FlushTask->EnsureCompletion();
			FlushTask.Reset();
		}
	}
}

void FAnalyticsPropertyStore::Reset()
{
	// Ensure any running async flush has completed.
	if (FlushTask.IsValid())
	{
		FlushTask->EnsureCompletion();
		FlushTask.Reset();
	}

	FileHandle.Reset();
	NameOffsetMap.Reset();
	StorageBuf.Empty(StorageBuf.Num());
	StorageReader.Seek(0);
	StorageWriter.Seek(0);
}

bool FAnalyticsPropertyStore::Create(const FString& Pathname, uint32 CapacityHint)
{
	UE::TScopeLock Lock(StoreLock);

	// Reset internal state.
	Reset();

	CapacityHint = FMath::Max(static_cast<uint32>(sizeof(AnalyticsPropertyStoreUtils::FAnalyticsStoreFileHeader)), CapacityHint);
	uint32 ReservedSize = FMath::Min(CapacityHint, 2u * 1024 * 1024); // Limit the reserved space to 2MB.
	StorageBuf.Reserve(ReservedSize - sizeof(AnalyticsPropertyStoreUtils::FAnalyticsStoreFileHeader));

	// Open the file in read/write mode (random access needed).
	FileHandle = TUniquePtr<IFileHandle>(FPlatformFileManager::Get().GetPlatformFile().OpenWrite(*Pathname, /*bAppend*/false, /*bAllowRead*/false));
	if (!FileHandle)
	{
		return false;
	}

	// Reserve the space in the file. Nothing else until the store is flushed the first time.
	uint8 Dummy = 0;
	FileHandle->Seek(ReservedSize - sizeof(Dummy));
	return FileHandle->Write(&Dummy, sizeof(Dummy));
}

bool FAnalyticsPropertyStore::Load(const FString& Pathname)
{
	UE::TScopeLock Lock(StoreLock);

	// Reset internal state.
	Reset();

	// Open the file in read/write mode (random access needed).
	TUniquePtr<IFileHandle> ScopedFileHandle(FPlatformFileManager::Get().GetPlatformFile().OpenWrite(*Pathname, /*bAppend*/true, /*bAllowRead*/true));
	if (!ScopedFileHandle)
	{
		return false;
	}

	// Ensure the file header data is there. [Version | Checksum | StoreDataSize]. The file can be larger that the stored data size.
	int64 FileSize = ScopedFileHandle->Size();
	if (FileSize < sizeof(AnalyticsPropertyStoreUtils::FAnalyticsStoreFileHeader))
	{
		return false;
	}

	// Read the header.
	ScopedFileHandle->Seek(0);
	AnalyticsPropertyStoreUtils::FAnalyticsStoreFileHeader FileHeader;
	ScopedFileHandle->Read(reinterpret_cast<uint8*>(&FileHeader), sizeof(FileHeader));

	// If the file format is incompatible.
	if (FileHeader.Version != AnalyticsPropertyStoreUtils::StoreFormatVersion)
	{
		return false;
	}

	// Read the stored data.
	StorageBuf.Empty(FileHeader.DataSize);
	StorageBuf.AddUninitialized(FileHeader.DataSize);
	ScopedFileHandle->Read(StorageBuf.GetData(), FileHeader.DataSize);

	// Compute the checksum from the stored data.
	uint8 Checksum[AnalyticsPropertyStoreUtils::Md5DigestSize];
	FMD5 Md5;
	Md5.Update(StorageBuf.GetData(), StorageBuf.Num());
	Md5.Final(Checksum);

	// If the checksums don't match.
	if (FMemory::Memcmp(FileHeader.Checksum, Checksum, AnalyticsPropertyStoreUtils::Md5DigestSize) != 0)
	{
		return false;
	}

	// Rebuild the key/offset map, reading the stream sequentially.
	FString Key;
	uint32 Offset;
	while (StorageReader.Tell() < StorageReader.TotalSize())
	{
		// Get the current offset.
		Offset = static_cast<uint32>(StorageReader.Tell());

		// Read the type.
		ETypeCode TypeCode;
		StorageReader.Serialize(&TypeCode, sizeof(ETypeCode));

		switch (RawType(TypeCode))
		{
			case ETypeCode::I32:
			case ETypeCode::U32:
			case ETypeCode::Flt:
			{
				static_assert(sizeof(uint32) == sizeof(int32) && sizeof(uint32) == sizeof(float), "Incompatible data size");
				StorageReader.Seek(StorageReader.Tell() + sizeof(uint32)); // Skip over the value, it is not needed.
				StorageReader << Key;
				if (Key.IsEmpty())
				{
					return false; // Invalid and not expected. Might be because the header or payload got corrupted before/during the write, likely a race condition in FAnalyticsPropertyStore::Flush()
				}
				else if (!IsDead(TypeCode))
				{
					NameOffsetMap.Emplace(Key, Offset);
				}
				break;
			}

			case ETypeCode::I64:
			case ETypeCode::U64:
			case ETypeCode::Dbl:
			case ETypeCode::Date:
			{
				static_assert(sizeof(uint64) == sizeof(int64) && sizeof(uint64) == sizeof(double) && sizeof(uint64) == sizeof(decltype(static_cast<FDateTime*>(nullptr)->GetTicks())), "Incompatible data size");
				StorageReader.Seek(StorageReader.Tell() + sizeof(uint64)); // Skip over the value, it is not needed.
				StorageReader << Key;
				if (Key.IsEmpty())
				{
					return false; // Invalid and not expected. Might be because the header or payload got corrupted before/during the write, likely a race condition in FAnalyticsPropertyStore::Flush()
				}
				else if (!IsDead(TypeCode))
				{
					NameOffsetMap.Emplace(Key, Offset);
				}
				break;
			}

			case ETypeCode::Bool:
			{
				StorageReader.Seek(StorageReader.Tell() + sizeof(bool)); // Skip over the value, it is not needed.
				StorageReader << Key;
				if (Key.IsEmpty())
				{
					return false; // Invalid and not expected. Might be because the header or payload got corrupted before/during the write, likely a race condition in FAnalyticsPropertyStore::Flush()
				}
				else if (!IsDead(TypeCode))
				{
					NameOffsetMap.Emplace(Key, Offset);
				}
				break;
			}

			case ETypeCode::Str:
			{
				uint32 CapacityInBytes;
				StorageReader.Seek(StorageReader.Tell() + sizeof(uint32)); // Skip over the size
				StorageReader.Serialize(&CapacityInBytes, sizeof(CapacityInBytes)); // Read the capacity.
				StorageReader.Seek(StorageReader.Tell() + CapacityInBytes); // Skip over the value and its extra capacity space if any.
				StorageReader << Key;
				if (Key.IsEmpty())
				{
					return false; // Invalid and not expected. Might be because the header or payload got corrupted before/during the write, likely a race condition in FAnalyticsPropertyStore::Flush()
				}
				else if (!IsDead(TypeCode))
				{
					NameOffsetMap.Emplace(Key, Offset);
				}
				break;
			}

			default:
				break;
		}
	}

	// Keep the file handle/open since the file content is valid.
	FileHandle = MoveTemp(ScopedFileHandle);
	return true;
}

uint32 FAnalyticsPropertyStore::Num() const
{
	UE::TScopeLock Lock(StoreLock);
	return NameOffsetMap.Num();
}

bool FAnalyticsPropertyStore::Contains(const FString& Key) const
{
	UE::TScopeLock Lock(StoreLock);
	return NameOffsetMap.Contains(Key);
}

bool FAnalyticsPropertyStore::Remove(const FString& Key)
{
	UE::TScopeLock Lock(StoreLock);
	uint32 Offset;
	if (NameOffsetMap.RemoveAndCopyValue(Key, Offset))
	{
		// Update the type code, adding the flag 'dead'.
		ETypeCode TypeCode;
		StorageReader.Seek(Offset);
		StorageReader.Serialize(&TypeCode, sizeof(ETypeCode));
		TypeCode |= ETypeCode::Dead;
		StorageWriter.Seek(Offset);
		StorageWriter.Serialize(&TypeCode, sizeof(ETypeCode));
		bFragmented = true;
		return true;
	}
	return false; // Was not found.
}

void FAnalyticsPropertyStore::RemoveAll()
{
	UE::TScopeLock Lock(StoreLock);
	NameOffsetMap.Reset();
	StorageReader.Seek(0);
	StorageWriter.Seek(0);
	StorageBuf.Empty(StorageBuf.Num());
	bFragmented = false;
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::Set(const FString& Key, int32 Value)
{
	UE::TScopeLock Lock(StoreLock);
	return SetFixedSizeValueInternal(Key, ETypeCode::I32, Value);
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::Set(const FString& Key, uint32 Value)
{
	UE::TScopeLock Lock(StoreLock);
	return SetFixedSizeValueInternal(Key, ETypeCode::U32, Value);
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::Set(const FString& Key, int64 Value)
{
	UE::TScopeLock Lock(StoreLock);
	return SetFixedSizeValueInternal(Key, ETypeCode::I64, Value);
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::Set(const FString& Key, uint64 Value)
{
	UE::TScopeLock Lock(StoreLock);
	return SetFixedSizeValueInternal(Key, ETypeCode::U64, Value);
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::Set(const FString& Key, float Value)
{
	UE::TScopeLock Lock(StoreLock);
	return SetFixedSizeValueInternal(Key, ETypeCode::Flt, Value);
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::Set(const FString& Key, double Value)
{
	UE::TScopeLock Lock(StoreLock);
	return SetFixedSizeValueInternal(Key, ETypeCode::Dbl, Value);
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::Set(const FString& Key, bool Value)
{
	UE::TScopeLock Lock(StoreLock);
	return SetFixedSizeValueInternal(Key, ETypeCode::Bool, Value);
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::Set(const FString& Key, const FString& Value, uint32 CharCountCapacityHint)
{
	UE::TScopeLock Lock(StoreLock);
	return SetStringValueInternal(Key, Value, CharCountCapacityHint);
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::Set(const FString& Key, const FDateTime& Value)
{
	UE::TScopeLock Lock(StoreLock);
	return SetDateTimeValueInternal(Key, Value);
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::Set(const FString& Key, int32 Value, const TFunction<bool(const int32*, const int32&)>& ConditionFn)
{
	UE::TScopeLock Lock(StoreLock);

	return AnalyticsPropertyStoreUtils::SetValue(
				[this, &Key](int32& OutValue) { return GetFixedSizeValueInternal(Key, ETypeCode::I32, OutValue); },
				[this, &Key](int32 InValue)   { return SetFixedSizeValueInternal(Key, ETypeCode::I32, InValue); },
				Value,
				ConditionFn);
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::Set(const FString& Key, uint32 Value, const TFunction<bool(const uint32*, const uint32&)>& ConditionFn)
{
	UE::TScopeLock Lock(StoreLock);

	return AnalyticsPropertyStoreUtils::SetValue(
				[this, &Key](uint32& OutValue) { return GetFixedSizeValueInternal(Key, ETypeCode::U32, OutValue); },
				[this, &Key](uint32 InValue)   { return SetFixedSizeValueInternal(Key, ETypeCode::U32, InValue); },
				Value,
				ConditionFn);
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::Set(const FString& Key, int64 Value, const TFunction<bool(const int64*, const int64&)>& ConditionFn)
{
	UE::TScopeLock Lock(StoreLock);

	return AnalyticsPropertyStoreUtils::SetValue(
				[this, &Key](int64& OutValue) { return GetFixedSizeValueInternal(Key, ETypeCode::I64, OutValue); },
				[this, &Key](int64 InValue)   { return SetFixedSizeValueInternal(Key, ETypeCode::I64, InValue); },
				Value,
				ConditionFn);
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::Set(const FString& Key, uint64 Value, const TFunction<bool(const uint64*, const uint64&)>& ConditionFn)
{
	UE::TScopeLock Lock(StoreLock);

	return AnalyticsPropertyStoreUtils::SetValue(
				[this, &Key](uint64& OutValue) { return GetFixedSizeValueInternal(Key, ETypeCode::U64, OutValue); },
				[this, &Key](uint64 InValue)   { return SetFixedSizeValueInternal(Key, ETypeCode::U64, InValue); },
				Value,
				ConditionFn);
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::Set(const FString& Key, float Value, const TFunction<bool(const float*, const float&)>& ConditionFn)
{
	UE::TScopeLock Lock(StoreLock);

	return AnalyticsPropertyStoreUtils::SetValue(
				[this, &Key](float& OutValue) { return GetFixedSizeValueInternal(Key, ETypeCode::Flt, OutValue); },
				[this, &Key](float InValue)   { return SetFixedSizeValueInternal(Key, ETypeCode::Flt, InValue); },
				Value,
				ConditionFn);
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::Set(const FString& Key, double Value, const TFunction<bool(const double*, const double&)>& ConditionFn)
{
	UE::TScopeLock Lock(StoreLock);

	return AnalyticsPropertyStoreUtils::SetValue(
				[this, &Key](double& OutValue) { return GetFixedSizeValueInternal(Key, ETypeCode::Dbl, OutValue); },
				[this, &Key](double InValue)   { return SetFixedSizeValueInternal(Key, ETypeCode::Dbl, InValue); },
				Value,
				ConditionFn);
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::Set(const FString& Key, bool Value, const TFunction<bool(const bool*, const bool&)>& ConditionFn)
{
	UE::TScopeLock Lock(StoreLock);

	return AnalyticsPropertyStoreUtils::SetValue(
				[this, &Key](bool& OutValue) { return GetFixedSizeValueInternal(Key, ETypeCode::Bool, OutValue); },
				[this, &Key](bool InValue)   { return SetFixedSizeValueInternal(Key, ETypeCode::Bool, InValue); },
				Value,
				ConditionFn);
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::Set(const FString& Key, const FString& Value, const TFunction<bool(const FString*, const FString&)>& ConditionFn)
{
	UE::TScopeLock Lock(StoreLock);

	return AnalyticsPropertyStoreUtils::SetValue(
				[this, &Key](FString& OutValue)      { return GetStringValueInternal(Key, OutValue); },
				[this, &Key](const FString& InValue) { return SetStringValueInternal(Key, InValue); },
				Value,
				ConditionFn);
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::Set(const FString& Key, const FDateTime& Value, const TFunction<bool(const FDateTime*, const FDateTime&)>& ConditionFn)
{
	UE::TScopeLock Lock(StoreLock);

	return AnalyticsPropertyStoreUtils::SetValue(
				[this, &Key](FDateTime& OutValue)      { return GetDateTimeValueInternal(Key, OutValue); },
				[this, &Key](const FDateTime& InValue) { return SetDateTimeValueInternal(Key, InValue); },
				Value,
				ConditionFn);
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::Update(const FString& Key, const TFunction<bool(int32&)>& UpdateFn)
{
	UE::TScopeLock Lock(StoreLock);

	return AnalyticsPropertyStoreUtils::UpdateValue<int32>(
				[this, &Key](int32& OutValue) { return GetFixedSizeValueInternal(Key, ETypeCode::I32, OutValue); },
				[this, &Key](int32 InValue)   { return SetFixedSizeValueInternal(Key, ETypeCode::I32, InValue); },
				UpdateFn);
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::Update(const FString& Key, const TFunction<bool(uint32&)>& UpdateFn)
{
	UE::TScopeLock Lock(StoreLock);

	return AnalyticsPropertyStoreUtils::UpdateValue<uint32>(
				[this, &Key](uint32& OutValue) { return GetFixedSizeValueInternal(Key, ETypeCode::U32, OutValue); },
				[this, &Key](uint32 InValue)   { return SetFixedSizeValueInternal(Key, ETypeCode::U32, InValue); },
				UpdateFn);
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::Update(const FString& Key, const TFunction<bool(int64&)>& UpdateFn)
{
	UE::TScopeLock Lock(StoreLock);

	return AnalyticsPropertyStoreUtils::UpdateValue<int64>(
				[this, &Key](int64& OutValue) { return GetFixedSizeValueInternal(Key, ETypeCode::I64, OutValue); },
				[this, &Key](int64 InValue)   { return SetFixedSizeValueInternal(Key, ETypeCode::I64, InValue); },
				UpdateFn);
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::Update(const FString& Key, const TFunction<bool(uint64&)>& UpdateFn)
{
	UE::TScopeLock Lock(StoreLock);

	return AnalyticsPropertyStoreUtils::UpdateValue<uint64>(
				[this, &Key](uint64& OutValue) { return GetFixedSizeValueInternal(Key, ETypeCode::U64, OutValue); },
				[this, &Key](uint64 InValue)   { return SetFixedSizeValueInternal(Key, ETypeCode::U64, InValue); },
				UpdateFn);
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::Update(const FString& Key, const TFunction<bool(float&)>& UpdateFn)
{
	UE::TScopeLock Lock(StoreLock);

	return AnalyticsPropertyStoreUtils::UpdateValue<float>(
				[this, &Key](float& OutValue) { return GetFixedSizeValueInternal(Key, ETypeCode::Flt, OutValue); },
				[this, &Key](float InValue)   { return SetFixedSizeValueInternal(Key, ETypeCode::Flt, InValue); },
				UpdateFn);
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::Update(const FString& Key, const TFunction<bool(double&)>& UpdateFn)
{
	UE::TScopeLock Lock(StoreLock);

	return AnalyticsPropertyStoreUtils::UpdateValue<double>(
				[this, &Key](double& OutValue) { return GetFixedSizeValueInternal(Key, ETypeCode::Dbl, OutValue); },
				[this, &Key](double InValue)   { return SetFixedSizeValueInternal(Key, ETypeCode::Dbl, InValue); },
				UpdateFn);
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::Update(const FString& Key, const TFunction<bool(bool&)>& UpdateFn)
{
	UE::TScopeLock Lock(StoreLock);

	return AnalyticsPropertyStoreUtils::UpdateValue<bool>(
				[this, &Key](bool& OutValue) { return GetFixedSizeValueInternal(Key, ETypeCode::Bool, OutValue); },
				[this, &Key](bool InValue)   { return SetFixedSizeValueInternal(Key, ETypeCode::Bool, InValue); },
				UpdateFn);
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::Update(const FString& Key, const TFunction<bool(FString&)>& UpdateFn)
{
	UE::TScopeLock Lock(StoreLock);

	return AnalyticsPropertyStoreUtils::UpdateValue<FString>(
				[this, &Key](FString& OutValue)      { return GetStringValueInternal(Key, OutValue); },
				[this, &Key](const FString& InValue) { return SetStringValueInternal(Key, InValue); },
				UpdateFn);
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::Update(const FString& Key, const TFunction<bool(FDateTime&)>& UpdateFn)
{
	UE::TScopeLock Lock(StoreLock);

	return AnalyticsPropertyStoreUtils::UpdateValue<FDateTime>(
				[this, &Key](FDateTime& OutValue)      { return GetDateTimeValueInternal(Key, OutValue); },
				[this, &Key](const FDateTime& InValue) { return SetDateTimeValueInternal(Key, InValue); },
				UpdateFn);
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::Get(const FString& Key, int32& OutValue) const
{
	UE::TScopeLock Lock(StoreLock);
	return GetFixedSizeValueInternal(Key, ETypeCode::I32, OutValue);
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::Get(const FString& Key, uint32& OutValue) const
{
	UE::TScopeLock Lock(StoreLock);
	return GetFixedSizeValueInternal(Key, ETypeCode::U32, OutValue);
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::Get(const FString& Key, int64& OutValue) const
{
	UE::TScopeLock Lock(StoreLock);
	return GetFixedSizeValueInternal(Key, ETypeCode::I64, OutValue);
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::Get(const FString& Key, uint64& OutValue) const
{
	UE::TScopeLock Lock(StoreLock);
	return GetFixedSizeValueInternal(Key, ETypeCode::U64, OutValue);
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::Get(const FString& Key, float& OutValue) const
{
	UE::TScopeLock Lock(StoreLock);
	return GetFixedSizeValueInternal(Key, ETypeCode::Flt, OutValue);
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::Get(const FString& Key, double& OutValue) const
{
	UE::TScopeLock Lock(StoreLock);
	return GetFixedSizeValueInternal(Key, ETypeCode::Dbl, OutValue);
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::Get(const FString& Key, bool& OutValue) const
{
	UE::TScopeLock Lock(StoreLock);
	return GetFixedSizeValueInternal(Key, ETypeCode::Bool, OutValue);
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::Get(const FString& Key, FString& OutValue) const
{
	UE::TScopeLock Lock(StoreLock);
	return GetStringValueInternal(Key, OutValue);
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::Get(const FString& Key, FDateTime& OutValue) const
{
	UE::TScopeLock Lock(StoreLock);
	return GetDateTimeValueInternal(Key, OutValue);
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::SetFixedSizeValueInternal(const FString& Key, ETypeCode TypeCode, const uint8* Value, uint32 Size)
{
	check(IsFixedSize(TypeCode));

	// NOTE: Format for records holding fixed-length value is: [TypeCode | Value | Key ]
	if (const uint32* RecordOffset = NameOffsetMap.Find(Key))
	{
		ETypeCode StoredValueType;
		StorageReader.Seek(*RecordOffset);
		StorageReader.Serialize(&StoredValueType, sizeof(ETypeCode));
		if (StoredValueType == TypeCode)
		{
			StorageWriter.Seek(*RecordOffset + sizeof(TypeCode));
			StorageWriter.Serialize(const_cast<uint8*>(Value), Size);
			return EStatusCode::Success;
		}
		return EStatusCode::BadType;
	}
	else
	{
		// Insert the new record at the end.
		int64 NewRecordOffset = StorageWriter.TotalSize();
		StorageWriter.Seek(NewRecordOffset);

		// Write the type.
		StorageWriter.Serialize(&TypeCode, sizeof(ETypeCode));

		// Write the value.
		StorageWriter.Serialize(const_cast<uint8*>(Value), Size);

		// Write the key. Let the FArchive encode the string.
		StorageWriter << const_cast<FString&>(Key);

		// Cache the offset to the newly inserted record.
		NameOffsetMap.Emplace(Key, IntCastChecked<uint32>(NewRecordOffset));
		return EStatusCode::Success;
	}
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::GetFixedSizeValueInternal(const FString& Key, ETypeCode TypeCode, uint8* OutValue, uint32 Size) const
{
	check(IsFixedSize(TypeCode));

	// NOTE: Format for records holding fixed-length value is: [TypeCode | Value | Key ]
	if (const uint32* RecordOffset = NameOffsetMap.Find(Key))
	{
		ETypeCode StoredTypeCode;
		StorageReader.Seek(*RecordOffset);
		StorageReader.Serialize(&StoredTypeCode, sizeof(ETypeCode));
		if (StoredTypeCode == TypeCode)
		{
			StorageReader.Serialize(OutValue, Size);
			return EStatusCode::Success;
		}
		return EStatusCode::BadType;
	}
	return EStatusCode::NotFound;
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::SetStringValueInternal(const FString& Key, const FString& Value, uint32 CharCountCapacityHint)
{
	// NOTE: Format for records holding strings is: [Type | SizeInBytes | CapacityInBytes | String-UTF16 | Key ]

	// Insert a variable-length value.
	auto InsertFn = [this](const FString& Key, ETypeCode TypeCode, const void* Bytes, uint32 ByteCount, uint32 Capacity)
	{
		// Insert the new record at the end.
		int64 InsertOffset = StorageWriter.TotalSize();
		StorageWriter.Seek(InsertOffset);

		// Write the value type.
		StorageWriter.Serialize(&TypeCode, sizeof(ETypeCode));

		// Write the data size.
		StorageWriter.Serialize(const_cast<uint32*>(&ByteCount), sizeof(uint32));

		// Write the capacity (in bytes) and reserve space if needed.
		StorageWriter.Serialize(&Capacity, sizeof(uint32));

		// Write the data.
		StorageWriter.Serialize(const_cast<void*>(Bytes), ByteCount);

		// Reserve extra capacity if needed.
		if (Capacity > ByteCount)
		{
			StorageWriter.Seek(StorageWriter.TotalSize() + Capacity - ByteCount); // Reserve extra space for the value.
		}

		// Write the key. Let the FArchive encode the string.
		StorageWriter << const_cast<FString&>(Key);

		// Cache the offset to the newly inserted record.
		NameOffsetMap.Emplace(Key, IntCastChecked<uint32>(InsertOffset));
		return EStatusCode::Success;
	};

	// Store the string in UTF16 with a null terminator. This is a no-op on platforms that are using a 16-bit TCHAR.
	FTCHARToUTF16 UTF16String(*Value, Value.Len() + 1);
	const void* Data = UTF16String.Get();
	uint32 DataSize = (UTF16String.Length() + 1) * sizeof(UTF16CHAR);

	if (const uint32* RecordOffset = NameOffsetMap.Find(Key))
	{
		// Ensure the types match.
		ETypeCode StoredValueType;
		StorageReader.Seek(*RecordOffset);
		StorageReader.Serialize(&StoredValueType, sizeof(ETypeCode));
		if (StoredValueType == ETypeCode::Str)
		{
			// Skip over the stored size.
			StorageReader.Seek(StorageReader.Tell() + sizeof(uint32));

			// Read the capacity.
			uint32 AvailableCapacity;
			StorageReader.Serialize(&AvailableCapacity, sizeof(uint32));

			// Can it be updated in-place?
			if (AvailableCapacity >= DataSize)
			{
				// Update the size.
				StorageWriter.Seek(*RecordOffset + sizeof(ETypeCode));
				StorageWriter.Serialize(&DataSize, sizeof(uint32));

				// Skip over the capacity (it did not change).
				StorageWriter.Seek(StorageWriter.Tell() + sizeof(uint32));

				// Update the value.
				StorageWriter.Serialize(const_cast<void*>(Data), DataSize);
				return EStatusCode::Success;
			}
			else
			{
				// Tombstone the current location.
				ETypeCode Tombstone = ETypeCode::Str | ETypeCode::Dead;
				StorageWriter.Seek(*RecordOffset);
				StorageWriter.Serialize(&Tombstone, sizeof(ETypeCode));
				bFragmented = true;

				// Relocate the record at the end.
				return InsertFn(Key, ETypeCode::Str, Data, DataSize, DataSize);
			}
		}
		return EStatusCode::BadType;
	}

	constexpr uint32 MaxExtraCapacityReserved = 2 * 1024;
	const uint32 RequestedCapacity = CharCountCapacityHint * static_cast<uint32>(sizeof(UTF16CHAR));
	return InsertFn(Key, ETypeCode::Str, Data, DataSize, RequestedCapacity <= DataSize ? DataSize : FMath::Min(RequestedCapacity, DataSize + MaxExtraCapacityReserved));
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::GetStringValueInternal(const FString& Key, FString& OutValue) const
{
	// NOTE: Format for records holding strings is: [Type | SizeInBytes | CapacityInBytes | UTF16String | Key ]

	if (const uint32* RecordOffset = NameOffsetMap.Find(Key))
	{
		ETypeCode StoredTypeCode;
		StorageReader.Seek(*RecordOffset);
		StorageReader.Serialize(&StoredTypeCode, sizeof(ETypeCode));
		if (StoredTypeCode == ETypeCode::Str)
		{
			// Read the size.
			uint32 DataSize;
			StorageReader.Serialize(&DataSize, sizeof(uint32));

			// Skip over the capacity.
			StorageReader.Seek(StorageReader.Tell() + sizeof(uint32));

			// Read the UTF16 string into a FString.
			uint32 CharCountWithNull = DataSize / sizeof(UTF16CHAR);
			OutValue.GetCharArray().Empty(CharCountWithNull);
			OutValue.GetCharArray().AddUninitialized(CharCountWithNull);
			auto Passthru = StringMemoryPassthru<UTF16CHAR>(OutValue.GetCharArray().GetData(), CharCountWithNull, CharCountWithNull);
			StorageReader.Serialize(Passthru.Get(), DataSize);
			Passthru.Get()[CharCountWithNull - 1] = '\0';
			Passthru.Apply();

			return EStatusCode::Success;
		}
		return EStatusCode::BadType;
	}
	return EStatusCode::NotFound;
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::SetDateTimeValueInternal(const FString& Key, const FDateTime& Value)
{
	int64 Ticks = Value.GetTicks();
	return SetFixedSizeValueInternal(Key, ETypeCode::Date, reinterpret_cast<const uint8*>(&Ticks), sizeof(Ticks));
}

IAnalyticsPropertyStore::EStatusCode FAnalyticsPropertyStore::GetDateTimeValueInternal(const FString& Key, FDateTime& OutValue) const
{
	int64 Ticks;
	EStatusCode Status = GetFixedSizeValueInternal(Key, ETypeCode::Date, Ticks);
	if (Status == EStatusCode::Success)
	{
		OutValue = FDateTime(Ticks);
	}
	return Status;
}

void FAnalyticsPropertyStore::Defragment()
{
	// NOTE: This is not implemented because we don't fragment enough the store in the current usage to benefit from defragmenting it.
	bFragmented = false;
}

void FAnalyticsPropertyStore::VisitAll(const TFunction<void(FAnalyticsEventAttribute&&)>& VisitFn) const
{
	UE::TScopeLock Lock(StoreLock);

	for (const TPair<FString, uint32>& NameOffsetPair : NameOffsetMap)
	{
		ETypeCode TypeCode;
		StorageReader.Seek(NameOffsetPair.Value);
		StorageReader << TypeCode;
		switch (RawType(TypeCode))
		{
			case ETypeCode::I32:
			{
				int32 V;
				StorageReader.Serialize(&V, sizeof(V));
				VisitFn(FAnalyticsEventAttribute(NameOffsetPair.Key, V));
				break;
			}

			case ETypeCode::U32:
			{
				uint32 V;
				StorageReader.Serialize(&V, sizeof(V));
				VisitFn(FAnalyticsEventAttribute(NameOffsetPair.Key, V));
				break;
			}

			case ETypeCode::I64:
			{
				int64 V;
				StorageReader.Serialize(&V, sizeof(V));
				VisitFn(FAnalyticsEventAttribute(NameOffsetPair.Key, V));
				break;
			}

			case ETypeCode::U64:
			{
				uint64 V;
				StorageReader.Serialize(&V, sizeof(V));
				VisitFn(FAnalyticsEventAttribute(NameOffsetPair.Key, V));
				break;
			}

			case ETypeCode::Flt:
			{
				float V;
				StorageReader.Serialize(&V, sizeof(V));
				VisitFn(FAnalyticsEventAttribute(NameOffsetPair.Key, V));
				break;
			}

			case ETypeCode::Dbl:
			{
				double V;
				StorageReader.Serialize(&V, sizeof(V));
				VisitFn(FAnalyticsEventAttribute(NameOffsetPair.Key, V));
				break;
			}

			case ETypeCode::Bool:
			{
				bool V;
				StorageReader.Serialize(&V, sizeof(V));
				VisitFn(FAnalyticsEventAttribute(NameOffsetPair.Key, V)); // Convert to "true" or "false".
				break;
			}

			case ETypeCode::Str:
			{
				FString V;
				GetStringValueInternal(NameOffsetPair.Key, V);
				VisitFn(FAnalyticsEventAttribute(NameOffsetPair.Key, V));
				break;
			}

			case ETypeCode::Date:
			{
				FDateTime V;
				GetDateTimeValueInternal(NameOffsetPair.Key, V);
				VisitFn(FAnalyticsEventAttribute(NameOffsetPair.Key, V.ToIso8601())); // Iso-8601 is the format used by Analytics in general for date/time.
				break;
			}

			default:
				break;
		}
	}
}

bool FAnalyticsPropertyStore::Flush(bool bAsync, const FTimespan& Timeout)
{
	if (!IsValid())
	{
		return false;
	}

	// If we are inside an AutoRTFM transaction, we can't flush yet, since this potentially involves writing
	// to disk and spinning up threads. 
	if (AutoRTFM::IsClosed())
	{
		// If we have the option to decline, just return false.
		if (Timeout < FTimespan::MaxValue())
		{
			return false;
		}

		// The caller expects the flush to succeed; queue up a blocking flush.
		AutoRTFM::PushOnCommitHandler(this, [this, bAsync]
		{
			Flush(bAsync, FTimespan::MaxValue());
		});
		return true;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FAnalyticsPropertyStore::Flush);

	// Local helper function to flush.
	auto FlushLockedFn = [this, bAsync](bool bNoWaiting)
	{
		// Ensure the previous task completed. Only one thread at a time can write the file.
		if (FlushTask.IsValid())
		{
			// If the task completed or can be successfully canceled.
			if (FlushTask->IsDone() || FlushTask->Cancel())
			{
				FlushTask.Reset();
			}
			else if (bNoWaiting)
			{
				return false; // Not allowed to wait. Don't flush.
			}
			else
			{
				// Wait for task to complete.
				FlushTask->EnsureCompletion();
				FlushTask.Reset();
			}
		}

		if (bAsync)
		{
			// Copy the storage buffer in a reusable buffer since there is at most one async flush at any point in time. This prevent allocating,
			// buffer sizes match most of the time.
			AsyncFlushDataCopy = StorageBuf;
			FlushTask = MakeUnique<FAsyncTask<FFlushWorker>>(*this);
			if (FlushTask)
			{
				FlushTask->StartBackgroundTask();
				return true; // Task started.
			}
		}

		// Flush synchronously.
		FlushInternal(StorageBuf);
		return true; // Flushed.
	};

	// If the caller doesn't want to wait for a previous execution (if any) to complete.
	if (Timeout.IsZero())
	{
		if (!StoreLock.TryLock())
		{
			return false;
		}
		ON_SCOPE_EXIT{ StoreLock.Unlock(); };
		return FlushLockedFn(/*bNoWaiting*/true);
	}

	// If the caller is ready to wait indefinitely to get its request started.
	if (Timeout == FTimespan::MaxValue())
	{
		UE::TScopeLock ScopeLock(StoreLock);
		return FlushLockedFn(/*bNoWaiting*/false);
	}
	
	// The caller is willing to wait some time to get its request processed.
	double StartTimeSecs = FPlatformTime::Seconds();
	do
	{
		if (StoreLock.TryLock())
		{
			ON_SCOPE_EXIT { StoreLock.Unlock(); };
			if (FlushLockedFn(/*bNoWaiting*/true))
			{
				return true;
			}
		}
		FPlatformProcess::Sleep(FloatCastChecked<float>(FTimespan::FromMilliseconds(1).GetTotalSeconds(), 1./1000.));
	} while (FPlatformTime::Seconds() - StartTimeSecs < Timeout.GetTotalSeconds());

	return false;
}

void FAnalyticsPropertyStore::FlushInternal(const TArray<uint8>& Data)
{
	AnalyticsPropertyStoreUtils::FAnalyticsStoreFileHeader Header;
	Header.Version = AnalyticsPropertyStoreUtils::StoreFormatVersion;
	Header.DataSize = Data.Num();

	// Compute the checksum.
	FMD5 Md5;
	Md5.Update(Data.GetData(), Data.Num());
	Md5.Final(Header.Checksum);

	// Write the file header.
	FileHandle->Seek(0);
	FileHandle->Write(reinterpret_cast<const uint8*>(&Header), sizeof(Header));

	// Write storage buf.
	FileHandle->Write(Data.GetData(), Data.Num());

	// Flush the File.
	FileHandle->Flush();
}

FAnalyticsPropertyStore::FFlushWorker::FFlushWorker(FAnalyticsPropertyStore& InStore)
	: Store(InStore)
{
}

void FAnalyticsPropertyStore::FFlushWorker::DoWork()
{
	// This expects that only one thread/task at the time flushes the data.
	Store.FlushInternal(Store.AsyncFlushDataCopy);
}
