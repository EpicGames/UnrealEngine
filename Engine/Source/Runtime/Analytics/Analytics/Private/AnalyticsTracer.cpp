// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnalyticsTracer.h"
#include "ProfilingDebugging/MiscTrace.h"

UE_DISABLE_OPTIMIZATION_SHIP
 
static void AggregateAttributes(TArray<FAnalyticsEventAttribute>& AggregatedAttibutes, const TArray<FAnalyticsEventAttribute>& Attributes)
{
	// Aggregates all attributes
	for (const FAnalyticsEventAttribute& Attribute : Attributes)
	{
		bool AttributeWasFound = false;

		for (FAnalyticsEventAttribute& AggregatedAttribute : AggregatedAttibutes)
		{
			if (Attribute.GetName() == AggregatedAttribute.GetName())
			{
				AggregatedAttribute += Attribute;

				// If we already have this attribute then great no more to do for this attribute
				AttributeWasFound = true;
				break;
			}
		}

		if (AttributeWasFound == false)
		{
			// No matching attribute so append
			AggregatedAttibutes.Add(Attribute);
		}
	}
}

void FAnalyticsSpan::SetProvider(TSharedPtr<IAnalyticsProvider> Provider)
{
	AnalyticsProvider = Provider;
}

void FAnalyticsSpan::SetStackDepth(uint32 Depth)
{
	StackDepth = Depth;
}

double FAnalyticsSpan::GetElapsedTime()
{
	return (FDateTime::UtcNow() - StartTime).GetTotalSeconds();
}

double FAnalyticsSpan::GetDuration() const 
{
	return Duration;
}

void FAnalyticsSpan::Start(const TArray<FAnalyticsEventAttribute>& AdditionalAttributes)
{
	// Create a new Guid for this flow, can we assume it is unique?
	Guid			= FGuid::NewGuid();
	Attributes		= AdditionalAttributes;
	ThreadId		= FPlatformTLS::GetCurrentThreadId();
	StartTime		= FDateTime::UtcNow();
	EndTime			= FDateTime::UtcNow();
	Duration		= 0;	
	IsActive		= true;

	// Always add the Span GUID to the attributes so we can join this span to other events
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Span_GUID"), Guid.ToString()));
}

bool FAnalyticsSpan::GetIsActive() const
{
	return IsActive;
}

void FAnalyticsSpan::End(const TArray<FAnalyticsEventAttribute>& AdditionalAttributes)
{
	// Only End the span once
	if (IsActive == false)
	{
		return;
	}

	// Calculate the duration
	EndTime = FDateTime::UtcNow();
	Duration = (EndTime - StartTime).GetTotalSeconds();
		
	// Add attributes and the the additional attributes to the current span attributes, these will get passed down to the child spans
	AddAttributes(AdditionalAttributes);

	const uint32 SpanSchemaVersion = 2;
	const FString SpanEventName = "Span";

	TArray<FAnalyticsEventAttribute> EventAttributes = Attributes;

	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("SchemaVersion"), SpanSchemaVersion));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("Span_Name"), Name.ToString()));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("Span_ThreadId"), ThreadId));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("Span_Depth"), StackDepth));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("Span_StartUTC"), StartTime.ToUnixTimestampDecimal()));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("Span_EndUTC"), EndTime.ToUnixTimestampDecimal()));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("Span_TimeInSec"), Duration));

	if (AnalyticsProvider.IsValid())
	{
		AnalyticsProvider->RecordEvent(SpanEventName, EventAttributes);
	}

	IsActive = false;
}

void FAnalyticsSpan::AddAttributes(const TArray<FAnalyticsEventAttribute>& AdditionalAttributes)
{
	AggregateAttributes(Attributes, AdditionalAttributes);
}

void FAnalyticsSpan::RecordEvent(const FString& EventName, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes)
{
	if (AnalyticsProvider.IsValid())
	{
		TArray<FAnalyticsEventAttribute> EventAttributes = Attributes;
		AggregateAttributes(EventAttributes, AdditionalAttributes);
		AnalyticsProvider->RecordEvent(EventName, EventAttributes);
	}
}

const FName& FAnalyticsSpan::GetName() const
{
	return Name;
}

FGuid FAnalyticsSpan::GetId() const
{
	return Guid;
}

const TArray<FAnalyticsEventAttribute>& FAnalyticsSpan::GetAttributes() const
{
	return Attributes;
}

uint32 FAnalyticsSpan::GetStackDepth() const
{
	return StackDepth;
}

void FAnalyticsTracer::SetProvider(TSharedPtr<IAnalyticsProvider> InProvider)
{
	AnalyticsProvider = InProvider;
}

TSharedPtr<IAnalyticsSpan> FAnalyticsTracer::GetCurrentSpan() const
{
	return ActiveSpanStack.Num() ? ActiveSpanStack.Top() : TSharedPtr<IAnalyticsSpan>();
}

void FAnalyticsTracer::StartSession()
{
	SessionSpan = StartSpan(TEXT("Session"), TSharedPtr<IAnalyticsSpan>());
}

void FAnalyticsTracer::EndSession()
{
	FScopeLock ScopeLock(&CriticalSection);

	EndSpan(SessionSpan);
	SessionSpan.Reset();

	// Stop any active spans, go from stack bottom first so parent spans will end their children
	while (ActiveSpanStack.Num())
	{
		EndSpan(ActiveSpanStack[0]);
	}

	ActiveSpanStack.Reset();

	AnalyticsProvider.Reset();
}

TSharedPtr<IAnalyticsSpan> FAnalyticsTracer::StartSpan(const FName NewSpanName, TSharedPtr<IAnalyticsSpan> ParentSpan, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes)
{
	FScopeLock ScopeLock(&CriticalSection);
	TSharedPtr<IAnalyticsSpan> NewSpan = MakeShared<FAnalyticsSpan>(NewSpanName);

	// Add the child to the parent's list of child spans
	if (ParentSpan.IsValid())
	{
		if (SpanHeirarchy.Find(ParentSpan->GetId()))
		{
			// Child list exists for this span already so append the new child for this span
			SpanHeirarchy[ParentSpan->GetId()].Emplace(NewSpan);
		}
		else
		{
			// Create a new child list for this span
			SpanHeirarchy.Emplace(ParentSpan->GetId(), { NewSpan });
		}
	}


	return StartSpanInternal(NewSpan, AdditionalAttributes) ? NewSpan : TSharedPtr<IAnalyticsSpan>();
}

void FAnalyticsTracer::BeginRegion(TSharedPtr<IAnalyticsSpan> Span)
{
	// This function is a temporary work around as UnrealInsights does not handle overlapping region with the same name.
	FName RegionName = Span->GetName();

	uint32 NameCounter = 0;

	while (RegionNames.Find(RegionName) != nullptr)
	{
		// Generate a unique region name for this span
		FNameBuilder NameBuilder(Span->GetName());
		RegionName = FName(FName(*FString::Printf(TEXT("%s%d"), *Span->GetName().ToString(), ++NameCounter)));
	}

	// Add the region name to the list by span ID
	RegionNames.Emplace(RegionName, Span->GetId());

	TRACE_BEGIN_REGION(*RegionName.ToString());
}

void FAnalyticsTracer::EndRegion(TSharedPtr<IAnalyticsSpan> Span)
{
	// Find the region, end it and remove from the region names
	for (TMap<FName, FGuid>::TConstIterator it(RegionNames); it; ++it)
	{
		// Slow match by ID on removal, fast match on creation
		if ((*it).Value == Span->GetId())
		{
			const FName& RegionName = (*it).Key;
			TRACE_END_REGION(*RegionName.ToString());
			RegionNames.Remove(RegionName);
			return;
		}
	}
}

bool FAnalyticsTracer::StartSpanInternal(TSharedPtr<IAnalyticsSpan> Span, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes)
{
	TSharedPtr<IAnalyticsSpan> LastAdddedActiveSpan = ActiveSpanStack.Num() ? ActiveSpanStack.Top() : TSharedPtr<IAnalyticsSpan>();
	Span->SetStackDepth(LastAdddedActiveSpan.IsValid() ? LastAdddedActiveSpan->GetStackDepth() + 1 : 0);
	Span->SetProvider(AnalyticsProvider);
	Span->Start(AdditionalAttributes);

	BeginRegion(Span);

	// Add span to active spans list
	ActiveSpanStack.Emplace(Span);

	return true;
}

bool FAnalyticsTracer::EndSpan(TSharedPtr<IAnalyticsSpan> Span, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes)
{
	FScopeLock ScopeLock(&CriticalSection);
	return EndSpanInternal(Span, AdditionalAttributes);
}

bool FAnalyticsTracer::EndSpanInternal(TSharedPtr<IAnalyticsSpan> Span, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes)
{
	if (Span.IsValid())
	{
		Span->End(AdditionalAttributes);

		EndRegion(Span);

		ActiveSpanStack.Remove(Span);

		// End any children of this span
		const TArray<TWeakPtr<IAnalyticsSpan>>* ChildSpans = SpanHeirarchy.Find(Span->GetId());
		
		if (ChildSpans != nullptr)
		{
			for (TWeakPtr<IAnalyticsSpan> ChildSpanWeakPtr : *ChildSpans)
			{
				// Pass the parent's attributes to the children as it ends
				EndSpanInternal(ChildSpanWeakPtr.Pin(), Span->GetAttributes());
			}
		}

		// Remove this span's child list 
		SpanHeirarchy.Remove(Span->GetId());
	}

	return false;
}

TSharedPtr<IAnalyticsSpan> FAnalyticsTracer::GetSessionSpan() const
{
	return SessionSpan;
}

TSharedPtr<IAnalyticsSpan> FAnalyticsTracer::GetSpanInternal(const FName Name)
{
	for (int32 i = 0; i < ActiveSpanStack.Num(); ++i)
	{
		TSharedPtr<IAnalyticsSpan> Span = ActiveSpanStack[i];
		
		if (Span.IsValid() && Span->GetName() == Name)
		{
			return Span;
		}
	}

	return TSharedPtr<IAnalyticsSpan>();
}

TSharedPtr<IAnalyticsSpan> FAnalyticsTracer::GetSpan(const FName Name)
{
	FScopeLock ScopeLock(&CriticalSection);
	return GetSpanInternal(Name);
}

UE_ENABLE_OPTIMIZATION_SHIP