// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interfaces/IAnalyticsProvider.h"

const TCHAR* LexToString(EAnalyticsRecordEventMode Mode)
{
    switch (Mode)
    {
        case EAnalyticsRecordEventMode::Cached: return TEXT("Cached");
        case EAnalyticsRecordEventMode::Immediate: return TEXT("Immediate");

        default:
        checkf(false, TEXT("Please, implement new modes here"));
        return TEXT("Unknown");
    }
}