// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/AutomationTest.h"
#include "Misc/Paths.h"
#include "AnalyticsPropertyStore.h"
#include "CoreGlobals.h"

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAnalyticsPropertyStoreAutomationTest, "System.Analytics.PropertyStore", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter | EAutomationTestFlags::MediumPriority)

bool FAnalyticsPropertyStoreAutomationTest::RunTest(const FString& Parameters)
{
	static const FString I32Key  = "MyInt32";
	static const FString U32Key  = "MyUInt32";
	static const FString I64Key  = "MyIint64";
	static const FString U64Key  = "MyUint64";
	static const FString FltKey  = "MyFloat";
	static const FString DblKey  = "MyDouble";
	static const FString BoolKey = "MyBool";
	static const FString Str1Key = "MyStr1";
	static const FString Str2Key = "MyStr2";
	static const FString DateKey = "MyDate";

	int32 Expected_I32 = 10;
	uint32 Expected_U32 = 20;
	int64 Expected_I64 = 30;
	uint64 Expected_U64 = 40;
	float Expected_Flt = 50.0f;
	double Expected_Dbl = 60.0;
	bool Expected_Bool = true;
	FString Expected_Str1 = "Hello World";
	FString Expected_Str2 = "Just Do It";
	FDateTime Expected_Date = FDateTime::UtcNow();

// When DO_CHECK is disabled (by default in Test and Shipping builds), the check() macro is a no-op.
// The variables below are only used in check().
// To avoid "error C4101: unreferenced local variable", only declare them when check() actually uses them.
#if DO_CHECK
	int32 Actual_I32;
	uint32 Actual_U32;
	int64 Actual_I64;
	uint64 Acutal_U64;
	float Actual_Flt;
	double Actual_Dbl;
	bool Actual_Bool;
	FString Actual_Str1;
	FString Actual_Str2;
	FDateTime Actual_Date;
	int32 NotFoundDummy;
#endif

	// Ensure the automation directory exists.
	IFileManager::Get().MakeDirectory(*FPaths::AutomationTransientDir(), /*Tree*/true);
	FString TestStorePathname = FPaths::AutomationTransientDir() / TEXT("AnalyticsPropertyStoreTest.bin");

	// Test invalid store - not created/not loaded.
	{
		FAnalyticsPropertyStore Store;
		check(!Store.IsValid());
		check(Store.Num() == 0);
	}

	// Create an empty store and reload it.
	{
		FAnalyticsPropertyStore Store;
		verify(Store.Create(TestStorePathname, /*CapacityHint*/2 * 1024));
		check(Store.IsValid());
		check(Store.Num() == 0);
		Store.Flush();
		verify(Store.Load(TestStorePathname));
		check(Store.IsValid());
		check(Store.Num() == 0);
	}

	// Test the basic operations.
	{
		FAnalyticsPropertyStore Store;
		verify(Store.Create(TestStorePathname, /*CapacityHint*/2 * 1024));
		check(Store.IsValid());
		check(Store.Num() == 0);

		// Write values.
		verify(Store.Set(I32Key,  Expected_I32) == IAnalyticsPropertyStore::EStatusCode::Success);
		verify(Store.Set(U32Key,  Expected_U32) == IAnalyticsPropertyStore::EStatusCode::Success);
		verify(Store.Set(I64Key,  Expected_I64) == IAnalyticsPropertyStore::EStatusCode::Success);
		verify(Store.Set(U64Key,  Expected_U64) == IAnalyticsPropertyStore::EStatusCode::Success);
		verify(Store.Set(FltKey,  Expected_Flt) == IAnalyticsPropertyStore::EStatusCode::Success);
		verify(Store.Set(DblKey,  Expected_Dbl) == IAnalyticsPropertyStore::EStatusCode::Success);
		verify(Store.Set(BoolKey, Expected_Bool) == IAnalyticsPropertyStore::EStatusCode::Success);
		verify(Store.Set(Str1Key, Expected_Str1, /*CapacityInChars*/25) == IAnalyticsPropertyStore::EStatusCode::Success);
		verify(Store.Set(DateKey, Expected_Date) == IAnalyticsPropertyStore::EStatusCode::Success);
		verify(Store.Set(Str2Key, Expected_Str2) == IAnalyticsPropertyStore::EStatusCode::Success);

		// Read values
		check(Store.Get(I32Key,  Actual_I32)  == IAnalyticsPropertyStore::EStatusCode::Success && Expected_I32 == Actual_I32);
		check(Store.Get(U32Key,  Actual_U32)  == IAnalyticsPropertyStore::EStatusCode::Success && Expected_U32 == Actual_U32);
		check(Store.Get(I64Key,  Actual_I64)  == IAnalyticsPropertyStore::EStatusCode::Success && Expected_I64 == Actual_I64);
		check(Store.Get(U64Key,  Acutal_U64)  == IAnalyticsPropertyStore::EStatusCode::Success && Expected_U64 == Acutal_U64);
		check(Store.Get(FltKey,  Actual_Flt)  == IAnalyticsPropertyStore::EStatusCode::Success && Expected_Flt == Actual_Flt);
		check(Store.Get(DblKey,  Actual_Dbl)  == IAnalyticsPropertyStore::EStatusCode::Success && Expected_Dbl == Actual_Dbl);
		check(Store.Get(BoolKey, Actual_Bool) == IAnalyticsPropertyStore::EStatusCode::Success && Expected_Bool == Actual_Bool);
		check(Store.Get(Str1Key, Actual_Str1) == IAnalyticsPropertyStore::EStatusCode::Success && Expected_Str1 == Actual_Str1);
		check(Store.Get(DateKey, Actual_Date) == IAnalyticsPropertyStore::EStatusCode::Success && Expected_Date == Actual_Date);
		check(Store.Get(Str2Key, Actual_Str2) == IAnalyticsPropertyStore::EStatusCode::Success && Expected_Str2 == Actual_Str2);

		// Key not found.
		check(Store.Get(TEXT("ShouldNotFound"), NotFoundDummy) == IAnalyticsPropertyStore::EStatusCode::NotFound);

		// Type mismatch.
		check(Store.Get(I32Key, Actual_U32)  == IAnalyticsPropertyStore::EStatusCode::BadType && Actual_U32 == Expected_U32);
		check(Store.Get(Str2Key, Actual_Dbl) == IAnalyticsPropertyStore::EStatusCode::BadType && Actual_Dbl == Expected_Dbl);

		// Contains.
		check(Store.Contains(I32Key));
		check(Store.Contains(U32Key));
		check(Store.Contains(I64Key));
		check(Store.Contains(U64Key));
		check(Store.Contains(FltKey));
		check(Store.Contains(DblKey));
		check(Store.Contains(BoolKey));
		check(Store.Contains(Str1Key));
		check(Store.Contains(DateKey));
		check(Store.Contains(Str2Key));
		check(!Store.Contains(TEXT("IDoNotExist")));

		// In-place update.
		Expected_I32 = 2000;
		verify(Store.Set(I32Key, Expected_I32) == IAnalyticsPropertyStore::EStatusCode::Success);
		check(Store.Get(I32Key, Actual_I32) == IAnalyticsPropertyStore::EStatusCode::Success);
		check(Expected_I32 == Actual_I32);

		// In-place string update. (Enough capacity was reserved)
		Expected_Str1 = TEXT("Hello World Update");
		verify(Store.Set(Str1Key, Expected_Str1) == IAnalyticsPropertyStore::EStatusCode::Success);
		check(Store.Get(Str1Key, Actual_Str1) == IAnalyticsPropertyStore::EStatusCode::Success && Actual_Str1 == Expected_Str1);
		check(Store.Get(Str2Key, Actual_Str2) == IAnalyticsPropertyStore::EStatusCode::Success && Expected_Str2 == Actual_Str2); // Ensure Str1 did not overwrite Str2.

		// Out-of-place string update. (Not enough capacity to perform in-place update).
		Expected_Str2 = TEXT("Hello World Extend");
		verify(Store.Set(Str2Key, Expected_Str2) == IAnalyticsPropertyStore::EStatusCode::Success);
		check(Store.Get(Str2Key, Actual_Str2) == IAnalyticsPropertyStore::EStatusCode::Success && Expected_Str2 == Actual_Str2);

		// Set conditionnaly - accepted
		++Expected_I32;
		verify(Store.Set(I32Key, Expected_I32, [](const int32* Actual, const int32& Proposed) { return *Actual < Proposed; }) == IAnalyticsPropertyStore::EStatusCode::Success);
		check(Store.Get(I32Key, Actual_I32) == IAnalyticsPropertyStore::EStatusCode::Success && Expected_I32 == Actual_I32);

		// Set conditionnaly - declined
		verify(Store.Set(FltKey, Expected_Flt - 50.0f, [](const float* Actual, const float& Proposed) { return *Actual < Proposed; }) == IAnalyticsPropertyStore::EStatusCode::Declined);
		check(Store.Get(FltKey, Actual_Flt) == IAnalyticsPropertyStore::EStatusCode::Success && Actual_Flt == Expected_Flt);

		// Set conditionnaly - wrong type
		verify(Store.Set(BoolKey, Expected_Dbl, [](const double* Actual, const double& Proposed) { check(false); return *Actual < Proposed; }) == IAnalyticsPropertyStore::EStatusCode::BadType);

		// Set conditionnaly - check params
		Expected_Str1 = TEXT("Hi, this is Joe, how are you?");
		verify(Store.Set(Str1Key, Expected_Str1, [&](const FString* Actual, const FString& Proposed) { check(*Actual == Actual_Str1); check(Proposed == Expected_Str1); return true; }) == IAnalyticsPropertyStore::EStatusCode::Success);
		check(Store.Get(Str1Key, Actual_Str1) == IAnalyticsPropertyStore::EStatusCode::Success && Actual_Str1 == Expected_Str1);

		// Update - accepted.
		++Expected_I64;
		verify(Store.Update(I64Key, [](int64& InOutStored) { ++InOutStored; return true; }) == IAnalyticsPropertyStore::EStatusCode::Success);
		check(Store.Get(I64Key, Actual_I64) == IAnalyticsPropertyStore::EStatusCode::Success && Actual_I64 == Expected_I64);

		// Update - declined.
		verify(Store.Update(DateKey, [](FDateTime& InOutStored) { InOutStored = FDateTime::UtcNow(); return false; }) == IAnalyticsPropertyStore::EStatusCode::Declined);
		check(Store.Get(DateKey, Actual_Date) == IAnalyticsPropertyStore::EStatusCode::Success && Actual_Date == Expected_Date);

		// Update - wrong type
		verify(Store.Update(Str1Key, [](uint32& InOutStored) { check(false); return true; }) == IAnalyticsPropertyStore::EStatusCode::BadType);

		// Update - check params
		Expected_Bool = !Expected_Bool;
		verify(Store.Update(BoolKey, [&](bool& InOutStored) { check(InOutStored == Actual_Bool); InOutStored = Expected_Bool; return true; }) == IAnalyticsPropertyStore::EStatusCode::Success);
		check(Store.Get(BoolKey, Actual_Bool) == IAnalyticsPropertyStore::EStatusCode::Success && Actual_Bool == Expected_Bool);

		// Test string conversion.
		Store.VisitAll([&](FAnalyticsEventAttribute&& Attr)
		{
			if (Attr.GetName() == I32Key)
			{
				check(Attr.GetValue() == LexToString(Expected_I32));
			}
			else if (Attr.GetName() == U32Key)
			{
				check(Attr.GetValue() == LexToString(Expected_U32));
			}
			else if (Attr.GetName() == I64Key)
			{
				check(Attr.GetValue() == LexToString(Expected_I64));
			}
			else if (Attr.GetName() == U64Key)
			{
				check(Attr.GetValue() == LexToString(Expected_U64));
			}
			else if (Attr.GetName() == FltKey)
			{
				check(Attr.GetValue() == FString::SanitizeFloat(Expected_Flt));
			}
			else if (Attr.GetName() == DblKey)
			{
				check(Attr.GetValue() == FString::SanitizeFloat(Expected_Dbl));
			}
			else if (Attr.GetName() == BoolKey)
			{
				check(Attr.GetValue() == LexToString(Expected_Bool));
			}
			else if (Attr.GetName() == Str1Key)
			{
				check(Attr.GetValue() == Expected_Str1);
			}
			else if (Attr.GetName() == DateKey)
			{
				check(Attr.GetValue() == Expected_Date.ToIso8601());
			}
			else if (Attr.GetName() == Str2Key)
			{
				check(Attr.GetValue() == Expected_Str2);
			}
			else
			{
				check(false); // This is an unexpected key...
			}
		});

		// Persist the changes above to disk.
		Store.Flush();
	}

	// Reload the store written above and verify the values.
	{
		FAnalyticsPropertyStore Store;
		verify(Store.Load(TestStorePathname));
		check(Store.IsValid());
		check(Store.Num() == 10);

		check(Store.Get(I32Key,  Actual_I32)  == IAnalyticsPropertyStore::EStatusCode::Success && Expected_I32 == Actual_I32);
		check(Store.Get(U32Key,  Actual_U32)  == IAnalyticsPropertyStore::EStatusCode::Success && Expected_U32 == Actual_U32);
		check(Store.Get(I64Key,  Actual_I64)  == IAnalyticsPropertyStore::EStatusCode::Success && Expected_I64 == Actual_I64);
		check(Store.Get(U64Key,  Acutal_U64)  == IAnalyticsPropertyStore::EStatusCode::Success && Expected_U64 == Acutal_U64);
		check(Store.Get(FltKey,  Actual_Flt)  == IAnalyticsPropertyStore::EStatusCode::Success && Expected_Flt == Actual_Flt);
		check(Store.Get(DblKey,  Actual_Dbl)  == IAnalyticsPropertyStore::EStatusCode::Success && Expected_Dbl == Actual_Dbl);
		check(Store.Get(BoolKey, Actual_Bool) == IAnalyticsPropertyStore::EStatusCode::Success && Expected_Bool == Actual_Bool);
		check(Store.Get(Str1Key, Actual_Str1) == IAnalyticsPropertyStore::EStatusCode::Success && Expected_Str1 == Actual_Str1);
		check(Store.Get(DateKey, Actual_Date) == IAnalyticsPropertyStore::EStatusCode::Success && Expected_Date == Actual_Date);
		check(Store.Get(Str2Key, Actual_Str2) == IAnalyticsPropertyStore::EStatusCode::Success && Expected_Str2 == Actual_Str2);

		// Test the remove API
		verify(Store.Remove(U32Key));
		verify(Store.Remove(I32Key));
		verify(Store.Remove(Str1Key));
		verify(Store.Remove(I64Key));
		verify(!Store.Remove(I64Key)); // Remove same key twice.
		verify(Store.Remove(U64Key));
		verify(Store.Remove(DblKey));
		verify(Store.Remove(FltKey));
		verify(Store.Remove(BoolKey));
		verify(Store.Remove(Str2Key));
		verify(!Store.Remove(Str2Key)); // Remove same key twice.
		verify(Store.Remove(DateKey));
		verify(!Store.Remove(TEXT("IDoNotExist")));

		check(Store.Num() == 0);

		// Ensure the key were deleted.
		check(!Store.Contains(I32Key));
		check(!Store.Contains(U32Key));
		check(!Store.Contains(I64Key));
		check(!Store.Contains(U64Key));
		check(!Store.Contains(FltKey));
		check(!Store.Contains(DblKey));
		check(!Store.Contains(BoolKey));
		check(!Store.Contains(Str1Key));
		check(!Store.Contains(DateKey));
		check(!Store.Contains(Str2Key));

		// Persist the empty store to disk.
		Store.Flush();
	}

	// Reload an empty store and check it doesn't have any key.
	{
		FAnalyticsPropertyStore Store;
		verify(Store.Load(TestStorePathname));
		check(Store.Num() == 0);

		check(!Store.Contains(I32Key));
		check(!Store.Contains(U32Key));
		check(!Store.Contains(I64Key));
		check(!Store.Contains(U64Key));
		check(!Store.Contains(FltKey));
		check(!Store.Contains(DblKey));
		check(!Store.Contains(BoolKey));
		check(!Store.Contains(Str1Key));
		check(!Store.Contains(DateKey));
		check(!Store.Contains(Str2Key));

		// Add if null.
		verify(Store.Set(I64Key, Expected_I64, [](const int64* Actual, const int64& Proposed) { return Actual == nullptr; }) == IAnalyticsPropertyStore::EStatusCode::Success);
		check(Store.Get(I64Key, Actual_I64) == IAnalyticsPropertyStore::EStatusCode::Success && Actual_I64 == Expected_I64);

		// Decline if null
		verify(Store.Set(I32Key, Expected_I32, [](const int32* Actual, const int32& Proposed) { return Actual != nullptr; }) == IAnalyticsPropertyStore::EStatusCode::Declined);
		check(Store.Get(I32Key, Actual_I32) == IAnalyticsPropertyStore::EStatusCode::NotFound && Actual_I32 == Expected_I32);

		check(Store.Num() == 1);

		verify(Store.Set(Str2Key, Expected_Str2) == IAnalyticsPropertyStore::EStatusCode::Success);
		verify(Store.Set(DblKey,  Expected_Dbl) == IAnalyticsPropertyStore::EStatusCode::Success);
		verify(Store.Set(BoolKey, Expected_Bool) == IAnalyticsPropertyStore::EStatusCode::Success);

		// Remove all.
		Store.RemoveAll();
		check(Store.Num() == 0);
		check(!Store.Contains(I64Key));
		check(!Store.Contains(DblKey));
		check(!Store.Contains(Str2Key));
		check(!Store.Contains(BoolKey))
	}

	// Corrupt a store.
	{
		TUniquePtr<FAnalyticsPropertyStore> Store = MakeUnique<FAnalyticsPropertyStore>();
		verify(Store->Create(TestStorePathname, /*CapacityHint*/2 * 1024));

		Store->Set(I32Key,  Expected_I32);
		Store->Set(U32Key,  Expected_U32);
		Store->Set(I64Key,  Expected_I64);
		Store->Set(U64Key,  Expected_U64);
		Store->Set(FltKey,  Expected_Flt);
		Store->Set(DblKey,  Expected_Dbl);
		Store->Set(BoolKey, Expected_Bool);
		Store->Set(Str1Key, Expected_Str1, /*CapacityInChars*/25);
		Store->Set(DateKey, Expected_Date);
		Store->Set(Str2Key, Expected_Str2);
		Store->Flush();
		Store.Reset(); // Close the file.

		TUniquePtr<IFileHandle> FileHandle(FPlatformFileManager::Get().GetPlatformFile().OpenWrite(*TestStorePathname, /*bAppend*/true, /*bAllowRead*/true));
		FileHandle->Seek(55);
		uint32 Corruption[] = { 66, 99, 1, 7, 44, 8888 }; // Random stream of bytes.
		FileHandle->Write(reinterpret_cast<uint8*>(Corruption), sizeof(Corruption));
		FileHandle.Reset();

		Store = MakeUnique<FAnalyticsPropertyStore>();
		verify(!Store->Load(TestStorePathname)); // Checksum should fails.
	}

	// Test store internal reset.
	{
		// Create the store.
		FAnalyticsPropertyStore Store;
		verify(Store.Create(TestStorePathname, /*CapacityHint*/2 * 1024));
		Store.Set(I32Key, Expected_I32);
		Store.Flush();

		// Reload the store -> This will reset the object.
		verify(Store.Load(TestStorePathname));
		check(Store.Get(I32Key, Actual_I32) == IAnalyticsPropertyStore::EStatusCode::Success && Expected_I32 == Actual_I32);

		// Recreate the store -> This will reset the object and create an empty store.
		verify(Store.Create(TestStorePathname, /*CapacityHint*/2 * 1024));
		check(Store.Get(I32Key, Actual_I32) == IAnalyticsPropertyStore::EStatusCode::NotFound);
		check(!Store.Contains(I32Key));
	}

	// Test flushing async - no timeout
	{
		// Create the store.
		FAnalyticsPropertyStore Store;
		verify(Store.Create(TestStorePathname, /*CapacityHint*/2 * 1024));
		Store.Set(I32Key, Expected_I32);
		Store.Flush(/*bAsync*/true);

		// Reload the store -> This will wait until the async task complates and reset the object.
		verify(Store.Load(TestStorePathname));
		check(Store.Get(I32Key, Actual_I32) == IAnalyticsPropertyStore::EStatusCode::Success && Expected_I32 == Actual_I32);
	}

	// Test flushing async - with timeout
	{
		// Create the store.
		FAnalyticsPropertyStore Store;
		verify(Store.Create(TestStorePathname, /*CapacityHint*/2 * 1024));
		Store.Set(I32Key, Expected_I32);
		Store.Flush(/*bAsync*/true, FTimespan::FromMilliseconds(100));

		// Reload the store -> This will wait until the async task complates and reset the object.
		verify(Store.Load(TestStorePathname));
		check(Store.Get(I32Key, Actual_I32) == IAnalyticsPropertyStore::EStatusCode::Success && Expected_I32 == Actual_I32);
	}


	IFileManager::Get().Delete(*TestStorePathname);

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS
