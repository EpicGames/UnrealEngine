// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnalyticsBuildType.h"
#include "AnalyticsProviderConfigurationDelegate.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Logging/LogMacros.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

ANALYTICS_API DECLARE_LOG_CATEGORY_EXTERN(LogAnalytics, Display, All);

class IAnalyticsProvider;
class IAnalyticsTracer;

/**
 * The public interface for interacting with analytics.
 * The basic usage is to call CreateAnalyticsProvider and supply a configuration delegate.
 * Specific analytics providers may choose to provide strongly-typed factory methods for configuration,
 * in which case you are free to call that directly if you know exactly what provider you will be using.
 * This class merely facilitates loosely bound provider configuration so the provider itself can be configured purely via config.
 * 
 * BuildType methods exist as a common way for an analytics provider to configure itself for debug/development/playtest/release scenarios.
 * Again, you can choose to ignore this info and provide a generic configuration delegate that does anything it wants.
 * 
 * To create an analytics provider using all the system defaults, simply call the static GetDefaultConfiguredProvider().
 * 
 */
class FAnalytics : public IModuleInterface
{
	//--------------------------------------------------------------------------
	// Module functionality
	//--------------------------------------------------------------------------
public:
	FAnalytics();
	virtual ~FAnalytics();

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline FAnalytics& Get()
	{
		return FModuleManager::LoadModuleChecked< FAnalytics >( "Analytics" );
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded( "Analytics" );
	}

	//--------------------------------------------------------------------------
	// Provider Factory functions.
	//--------------------------------------------------------------------------
public:
	/**
	 * Factory function to create a specific analytics provider by providing the string name of the provider module, which will be dynamically loaded.
	 * @param ProviderModuleName	The name of the module that contains the specific provider. It must be the primary module interface.
	 * @param GetConfigvalue	Delegate used to configure the provider. The provider will call this delegate once for each key it requires for configuration.
	 * @returns		the analytics provider instance that was created. Could be NULL if initialization failed.
	 */
	virtual TSharedPtr<IAnalyticsProvider> CreateAnalyticsProvider(const FName& ProviderModuleName, const FAnalyticsProviderConfigurationDelegate& GetConfigValue);

	virtual TSharedPtr<IAnalyticsTracer> CreateAnalyticsTracer();
	/**
	 * Creates an instance of the default configured analytics provider.
	 * Default is determined by GetDefaultProviderModuleName and a default constructed ConfigFromIni instance.
	 * 
	 * This implementation is kept in the header file to illustrate how providers are created using configuration delegates.
	 */
	virtual TSharedPtr<IAnalyticsProvider> GetDefaultConfiguredProvider()
	{
		FAnalytics::ConfigFromIni AnalyticsConfig;                     // configure using the default INI sections.
		return FAnalytics::Get().CreateAnalyticsProvider(              // call the factory function
			FAnalytics::ConfigFromIni::GetDefaultProviderModuleName(), // use the default config to find the provider name
			FAnalyticsProviderConfigurationDelegate::CreateRaw(        // bind to the delegate using default INI sections
				&AnalyticsConfig,                                      // use default INI config sections
				&FAnalytics::ConfigFromIni::GetValue));                
	}

	//--------------------------------------------------------------------------
	// Configuration helper classes and methods
	//--------------------------------------------------------------------------
public:
	/** 
	 * A common way of configuring is from Inis, so this class supports that notion directly by providing a
	 * configuration class with a method suitable to be used as an FAnalyticsProviderConfigurationDelegate
	 * that reads values from the specified ini and section (based on the BuildType).
	 * Also provides a default location to store a provider name, accessed via GetDefaultProviderModuleName().
	 */
	struct ConfigFromIni
	{
		/** 
		 * Create a config using the default values:
		 * IniName - GEngineIni
		 * SectionName (Development) = AnalyticsDevelopment
		 * SectionName (Debug) = AnalyticsDebug
		 * SectionName (Test) = AnalyticsTest
		 * SectionName (Release) = Analytics
		 */
		ConfigFromIni()
			:IniName(GEngineIni)
		{
			SetSectionNameByBuildType(GetAnalyticsBuildType());
		}

		/** Create a config AS IF the BuildType matches the one passed in. */
		explicit ConfigFromIni(EAnalyticsBuildType InBuildType)
			:IniName(GEngineIni)
		{
			SetSectionNameByBuildType(InBuildType);
		}

		/** Create a config, specifying the Ini name and a single section name for all build types. */
		ConfigFromIni(const FString& InIniName, const FString& InSectionName)
			:IniName(InIniName)
			,SectionName(InSectionName) 
		{
		}

		/** Create a config, specifying the Ini name and the section name for each build type. */
		ConfigFromIni(const FString& InIniName, const FString& SectionNameDevelopment, const FString& SectionNameDebug, const FString& SectionNameTest, const FString& SectionNameRelease)
			:IniName(InIniName)
		{
			EAnalyticsBuildType BuildType = GetAnalyticsBuildType();
			SectionName = BuildType == EAnalyticsBuildType::Release
				? SectionNameRelease
				: BuildType == EAnalyticsBuildType::Debug
					? SectionNameDebug
					: BuildType == EAnalyticsBuildType::Test
						? SectionNameTest
						: SectionNameDevelopment;
		}

		/** Method that can be bound to an FAnalyticsProviderConfigurationDelegate. */
		FString GetValue(const FString& KeyName, bool bIsRequired) const
		{
			return FAnalytics::Get().GetConfigValueFromIni(IniName, SectionName, KeyName, bIsRequired);
		}

		/** 
		 * Reads the ProviderModuleName key from the Analytics section of GEngineIni,
		 * which is the default, preferred location to look for the analytics provider name.
		 * This is purely optional, and you can store that information anywhere you want
		 * or merely hardcode the provider module.
		 */
		static FName GetDefaultProviderModuleName()
		{
			FString ProviderModuleName;
			GConfig->GetString(TEXT("Analytics"), TEXT("ProviderModuleName"), ProviderModuleName, GEngineIni);
			return FName(*ProviderModuleName);
		}

		/** Allows setting the INI section name based on the build type passed in. Allows access to the default section values when the application chooses the build type itself. */
		void SetSectionNameByBuildType(EAnalyticsBuildType InBuildType)
		{
			SectionName = InBuildType == EAnalyticsBuildType::Release
				? TEXT("Analytics") 
				: InBuildType == EAnalyticsBuildType::Debug
					? TEXT("AnalyticsDebug") 
					: InBuildType == EAnalyticsBuildType::Test
						? TEXT("AnalyticsTest") 
						: TEXT("AnalyticsDevelopment");
		}

		/** Ini file name to find the config values. */
		FString IniName;
		/** Section name in the Ini file in which to find the keys. The KeyNames should match the field name in the Config object. */
		FString SectionName;
	};

	/**
	 * Helper for reading configuration values from an INI file (which will be a common scenario). 
	 * This is exposed here so we're not exporting more classes from the module. It's merely a helper for ConfigFromIni struct above.
	 */
	virtual FString GetConfigValueFromIni(const FString& IniName, const FString& SectionName, const FString& KeyName, bool bIsRequired);

	/**
	 * Helper for writing configuration values from to an INI file (which will be a common scenario). 
	 */
	virtual void WriteConfigValueToIni(const FString& IniName, const FString& SectionName, const FString& KeyName, const FString& Value);

private:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};
