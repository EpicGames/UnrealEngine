// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/** Defines the different build types from an analytics perspective. Used to determine how to configure the provider. */
enum class EAnalyticsBuildType : uint8
{
	/**
	* Analytics go into a "slush" account that isn't meant to be representative.
	* This is the default mode.
	*/
	Development,
	/**
	* Test mode for playtests and other runs where the data collected will be semi-representative of actual gameplay. Should be routed to a test, or "representative data" account.
	* Use -TESTANALYTICS command line to trigger this mode.
	*/
	Test,
	/**
	* Debug mode where analytics should go to the Swrve debug console. Used for feature development and QA testing, since the events are visible on the debug console immediately.
	* Use -DEBUGANALYTICS command line to trigger this mode (overrides -TESTANALYTICS).
	*/
	Debug,
	/**
	* BuildType that should be used by the shipping game. UE_BUILD_SHIPPING builds use this mode (or can use -RELEASEANALYTICS cmdline to force it).
	*/
	Release,
};

/** Get the analytics build type. Generally used to determine the keys to use to configure an analytics provider. */
ANALYTICS_API EAnalyticsBuildType GetAnalyticsBuildType();

/** Convert an EAnalyticsBuildType to string. */
ANALYTICS_API const TCHAR* AnalyticsBuildTypeToString(EAnalyticsBuildType Type);


