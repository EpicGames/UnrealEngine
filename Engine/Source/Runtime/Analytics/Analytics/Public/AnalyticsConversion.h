// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/** Helpers for converting various common types to strings that analytics providers can consume. */

/** Lexical conversion. Allow any type that we have a Lex for. Can't use universal references here because it then eats all non-perfect matches for the array and TMap conversions below, which we want to use a custom, analytics specific implementation for. */
template <typename T>
inline decltype(auto) AnalyticsConversionToString(const T& Value)
{
	return LexToString(Value);
}
/** Make sure we have a direct implementation for moving FStrings as we definitely don't want to copy them as the above const-ref template will cause. */
inline FString AnalyticsConversionToString(FString&& Value)
{
	return MoveTemp(Value);
}
inline FString AnalyticsConversionToString(const FString& Value)
{
	return Value;
}

inline FString AnalyticsConversionToString(FStringView Value)
{
	return FString(Value);
}

#define ANALYTICS_MAX_FLOAT_VALUE_BEFORE_EXPONENTIAL_NOTATION (1.0e+15F)

inline FString AnalyticsConversionToString(float Value)
{
	if (FPlatformMath::IsFinite(Value))
	{
		if (Value > ANALYTICS_MAX_FLOAT_VALUE_BEFORE_EXPONENTIAL_NOTATION || Value < -ANALYTICS_MAX_FLOAT_VALUE_BEFORE_EXPONENTIAL_NOTATION)
		{
			return FString::Printf(TEXT("%.9e"), Value);
		}
		else
		{
			return LexToSanitizedString(Value);
		}
	}
	else
	{
		// This function exists to convert numbers to Json-compatible strings for analytics. Json doesn't support INF or NAN so we will just send null
		return "null";
	}
}
inline FString AnalyticsConversionToString(double Value)
{
	if (FPlatformMath::IsFinite(Value))
	{
		if (Value > ANALYTICS_MAX_FLOAT_VALUE_BEFORE_EXPONENTIAL_NOTATION || Value < -ANALYTICS_MAX_FLOAT_VALUE_BEFORE_EXPONENTIAL_NOTATION)
		{
			return FString::Printf(TEXT("%.9e"), Value);
		}
		else
		{
			return LexToSanitizedString(Value);
		}
	}
	else
	{
		// This function exists to convert numbers to Json-compatible strings for analytics. Json doesn't support INF or NAN so we will just send null
		return "null";
	}
}

/** Array conversion. Creates comma-separated list. */
template <typename T, typename AllocatorType>
FString AnalyticsConversionToString(const TArray<T, AllocatorType>& ValueArray)
{
	FString Result;
	// Serialize the array into "value1,value2,..." format
	for (const T& Value : ValueArray)
	{
		Result += AnalyticsConversionToString(Value);
		Result += TEXT(",");
	}
	// Remove the trailing comma (LeftChop will ensure an empty container won't crash here).
	Result.LeftChopInline(1, EAllowShrinking::No);
	return Result;
}

/** Array conversion. Creates comma-separated list. */
template <typename T>
FString AnalyticsConversionToString(const TArrayView<T>& ValueArray)
{
	FString Result;
	// Serialize the array into "value1,value2,..." format
	for (const T& Value : ValueArray)
	{
		Result += AnalyticsConversionToString(Value);
		Result += TEXT(",");
	}
	// Remove the trailing comma (LeftChop will ensure an empty container won't crash here).
	Result.LeftChopInline(1, EAllowShrinking::No);
	return Result;
}

/** Map conversion. Creates comma-separated list. Creates comma-separated list with colon-separated key:value pairs. */
template<typename KeyType, typename ValueType, typename Allocator, typename KeyFuncs>
FString AnalyticsConversionToString(const TMap<KeyType, ValueType, Allocator, KeyFuncs>& ValueMap)
{
	FString Result;
	// Serialize the map into "key:value,..." format
	for (auto& KVP : ValueMap)
	{
		Result += AnalyticsConversionToString(KVP.Key);
		Result += TEXT(":");
		Result += AnalyticsConversionToString(KVP.Value);
		Result += TEXT(",");
	}
	// Remove the trailing comma (LeftChop will ensure an empty container won't crash here).
	Result.LeftChopInline(1, EAllowShrinking::No);
	return Result;
}

/** Deprecated namespace. Forwards on to global equivalents for backwards compatibility. See functions above. Namespace was deprecated for the same reason as Lex::ToString -> LexToString. See UnrealString.h for details. */
namespace AnalyticsConversion
{
	template <typename T>
	inline decltype(auto) ToString(T&& Value)
	{
		return AnalyticsConversionToString(Forward<T>(Value));
	}
}