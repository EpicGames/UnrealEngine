// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnalyticsConversion.h"

struct FJsonNull
{
};

inline const TCHAR* LexToString(FJsonNull)
{
	return TEXT("null");
}

struct FJsonFragment
{
	/**
	 * Stores a JSON string, which is assumed to be correct JSON. If an empty string is passed, it will become "null" in LexToString.
	 */
	explicit FJsonFragment(FString&& StringRef) : FragmentString(MoveTemp(StringRef)) {}
	FString FragmentString;
};

inline const TCHAR* LexToString(const FJsonFragment& Fragment)
{
	if (Fragment.FragmentString.IsEmpty())
		return LexToString(FJsonNull{});
	return *Fragment.FragmentString;
}

inline FString LexToString(FJsonFragment&& Fragment)
{
	if (Fragment.FragmentString.IsEmpty())
		return LexToString(FJsonNull{});
	return MoveTemp(Fragment.FragmentString);
}



/**
 * Struct to hold key/value pairs that will be sent as attributes along with analytics events.
 * All values are actually strings, but we provide a convenient constructor that relies on ToStringForAnalytics() to 
 * convert common types. 
 */
struct FAnalyticsEventAttribute
{
	UE_DEPRECATED(4.26, "This property has been deprecated, use GetName() instead")
	const FString AttrName;

	UE_DEPRECATED(4.26, "This property has been deprecated, use GetValue() instead")
	const FString AttrValueString;
	UE_DEPRECATED(4.26, "This property has been deprecated, use GetValue() instead. You cannot recover the original non-string value anymore")
	const double AttrValueNumber;
	UE_DEPRECATED(4.26, "This property has been deprecated, use GetValue() instead. You cannot recover the original non-string value anymore")
	const bool AttrValueBool;

	enum class AttrTypeEnum
	{
		String,
		Number,
		Boolean,
		Null,
		JsonFragment
	};
	UE_DEPRECATED(4.26, "This property has been deprecated, use IsJsonFragment or GetValue instead")
	const AttrTypeEnum AttrType;

	template <typename ValueType>
	FAnalyticsEventAttribute(FString InName, ValueType&& InValue);

	const FString& GetName() const;
	const FString& GetValue() const;
	bool IsJsonFragment() const;

	/** Allow setting value for any type that supports LexToString */
	template<typename ValueType>
	void SetValue(ValueType&& InValue);

	/** Default ctor since we declare a custom ctor. */
	FAnalyticsEventAttribute();
	~FAnalyticsEventAttribute();

	/** Reinstate the default copy ctor because that one still works fine. */
	FAnalyticsEventAttribute(const FAnalyticsEventAttribute& RHS);

	/** Hack to allow copy ctor using an rvalue-ref. This class only "sort of" acts like an immutable class because the const members prevents assignment, which was not intended when this code was changed. */
	FAnalyticsEventAttribute(FAnalyticsEventAttribute&& RHS);

	/** Hack to allow assignment. This class only "sort of" acts like an immutable class because the const members prevents assignment, which was not intended when this code was changed. */
	FAnalyticsEventAttribute& operator=(const FAnalyticsEventAttribute& RHS);

	/** Hack to allow assignment. This class only "sort of" acts like an immutable class because the const members prevents assignment, which was not intended when this code was changed. */
	FAnalyticsEventAttribute& operator=(FAnalyticsEventAttribute&& RHS);

	/** ALlow aggregation of attributes */
	FAnalyticsEventAttribute& operator+=(const FAnalyticsEventAttribute& RHS);

	/** ALlow aggregation of attributes */
	FAnalyticsEventAttribute& operator+(const FAnalyticsEventAttribute& RHS);

	/** If you need the old AttrValue behavior (i.e. stringify everything), call this function instead. */
	UE_DEPRECATED(4.26, "This property has been deprecated, use GetValue() instead")
	FString ToString() const;

	/** Legacy support for old RecordEventJson API. Don't call this directly. */
	UE_DEPRECATED(4.26, "This property is used to support the deprecated APIs, construct Json values using FJsonFragment instead")
	void SwitchToJsonFragment();

	static bool IsValidAttributeName(const FString& InName)
	{
		return !InName.IsEmpty() && InName != TEXT("EventName") && InName != TEXT("DateOffset");
	}

private:
	FString& CheckName(FString& InName)
	{
		// These are reserved names in our environment. Enforce things don't use it.
		check(IsValidAttributeName(InName));
		return InName;
	}
};


// The implementation of this class references deprecated members. Don't fire warnings for these.
// For this reason we actually implement the entire class out-of-line, but still in the header files, so we can wrap
// all the implementations in DISABLE macro easily.
PRAGMA_DISABLE_DEPRECATION_WARNINGS

inline FAnalyticsEventAttribute::FAnalyticsEventAttribute() 
: AttrName()
, AttrValueString()
, AttrValueNumber(0)
, AttrValueBool(false)
, AttrType(AttrTypeEnum::String)
{

}

inline FAnalyticsEventAttribute::~FAnalyticsEventAttribute() = default;

inline FAnalyticsEventAttribute::FAnalyticsEventAttribute(const FAnalyticsEventAttribute& RHS) = default;

inline FAnalyticsEventAttribute::FAnalyticsEventAttribute(FAnalyticsEventAttribute&& RHS) : AttrName(MoveTemp(const_cast<FString&>(RHS.AttrName)))
, AttrValueString(MoveTemp(const_cast<FString&>(RHS.AttrValueString)))
// no need to use MoveTemp on intrinsic types.
, AttrValueNumber(RHS.AttrValueNumber)
, AttrValueBool(RHS.AttrValueBool)
, AttrType(RHS.AttrType)
{

}

template <typename ValueType>
inline FAnalyticsEventAttribute::FAnalyticsEventAttribute(FString InName, ValueType&& InValue)
: AttrName(MoveTemp(CheckName(InName)))
, AttrValueString(AnalyticsConversionToString(Forward<ValueType>(InValue)))
, AttrValueNumber(0)
, AttrValueBool(false)
, AttrType(TIsArithmetic<typename TDecay<ValueType>::Type>::Value || std::is_same_v<typename TDecay<ValueType>::Type, FJsonNull> || std::is_same_v<typename TDecay<ValueType>::Type, FJsonFragment> ? AttrTypeEnum::JsonFragment : AttrTypeEnum::String)
{

}


inline FAnalyticsEventAttribute& FAnalyticsEventAttribute::operator=(const FAnalyticsEventAttribute& RHS)
{
	if (&RHS == this)
	{
		return *this;
	}

	const_cast<FString&>(AttrName) = RHS.AttrName;
	const_cast<FString&>(AttrValueString) = RHS.AttrValueString;
	const_cast<double&>(AttrValueNumber) = RHS.AttrValueNumber;
	const_cast<bool&>(AttrValueBool) = RHS.AttrValueBool;
	const_cast<AttrTypeEnum&>(AttrType) = RHS.AttrType;
	return *this;
}

inline FAnalyticsEventAttribute& FAnalyticsEventAttribute::operator+=(const FAnalyticsEventAttribute& RHS)
{
	return *this+RHS;
}

inline FAnalyticsEventAttribute& FAnalyticsEventAttribute::operator+(const FAnalyticsEventAttribute& RHS)
{
	if (&RHS == this)
	{
		return *this;
	}

	const_cast<double&>(AttrValueNumber) += RHS.AttrValueNumber;
	return *this;
}

inline FAnalyticsEventAttribute& FAnalyticsEventAttribute::operator=(FAnalyticsEventAttribute&& RHS)
{
	if (&RHS == this)
	{
		return *this;
	}

	const_cast<FString&>(AttrName) = MoveTemp(const_cast<FString&>(RHS.AttrName));
	const_cast<FString&>(AttrValueString) = MoveTemp(const_cast<FString&>(RHS.AttrValueString));
	// no need to use MoveTemp on intrinsic types.
	const_cast<double&>(AttrValueNumber) = RHS.AttrValueNumber;
	const_cast<bool&>(AttrValueBool) = RHS.AttrValueBool;
	const_cast<AttrTypeEnum&>(AttrType) = RHS.AttrType;
	return *this;
}

inline FString FAnalyticsEventAttribute::ToString() const
{
	return GetValue();
}

inline const FString& FAnalyticsEventAttribute::GetName() const
{
	return AttrName;
}

inline const FString& FAnalyticsEventAttribute::GetValue() const
{
	return AttrValueString;
}

inline bool FAnalyticsEventAttribute::IsJsonFragment() const
{
	return AttrType == AttrTypeEnum::JsonFragment;
}

template<typename ValueType>
inline void FAnalyticsEventAttribute::SetValue(ValueType&& InValue)
{
	const_cast<FString&>(AttrValueString) = AnalyticsConversionToString(Forward<ValueType>(InValue));
	const_cast<AttrTypeEnum&>(AttrType) = TIsArithmetic<typename TDecay<ValueType>::Type>::Value || std::is_same_v<typename TDecay<ValueType>::Type, FJsonNull> || std::is_same_v<typename TDecay<ValueType>::Type, FJsonFragment> ? AttrTypeEnum::JsonFragment : AttrTypeEnum::String;
}

inline void FAnalyticsEventAttribute::SwitchToJsonFragment()
{
	const_cast<AttrTypeEnum&>(AttrType) = AttrTypeEnum::JsonFragment;
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

/** Helper functions for MakeAnalyticsEventAttributeArray. */
namespace ImplMakeAnalyticsEventAttributeArray
{
	// AddElement actually adds an element to the attributes array and is flagged no-inline so the array insert/value conversion
	// code is generated once per type combination.
	template <typename Allocator, typename KeyType, typename ValueType>
	FORCENOINLINE void AddElement(TArray<FAnalyticsEventAttribute, Allocator>& Attrs, const KeyType& Key, const ValueType& Value)
	{
		static_assert(std::is_convertible_v<KeyType, FString>, "Keys must be convertible to `FString`!");
		Attrs.Emplace(FString{Key}, Value);
	}

	template <typename Allocator>
    FORCEINLINE void MakeArray(TArray<FAnalyticsEventAttribute, Allocator>& Attrs)
    {
    }

	// MakeArray is just a helper whose only purpose is to generate a sequence of AddElement calls and is meant to be always inlined.
    template <typename Allocator, typename KeyType, typename ValueType, typename... RemainingArgTypes>
    FORCEINLINE void MakeArray(TArray<FAnalyticsEventAttribute, Allocator>& Attrs, KeyType&& Key, ValueType&& Value, RemainingArgTypes&& ...RemainingArgs)
	{
		static_assert(sizeof...(RemainingArgs) % 2 == 0, "Must pass an even number of arguments.");

		// Decay here to decay string literals into const char*/const TCHAR* (losing the information about their length)
		// to limit the number of unique instantiations of AddElement.
		AddElement<Allocator, std::decay_t<KeyType>, std::decay_t<ValueType>>(Attrs, Forward<KeyType>(Key), Forward<ValueType>(Value));
		MakeArray(Attrs, Forward<RemainingArgTypes>(RemainingArgs)...);
	}
}

/** Helper to create an array of attributes using a single expression. Reserves the necessary space in advance. There must be an even number of arguments, one for each key/value pair. */
template <typename Allocator = FDefaultAllocator, typename...ArgTypes>
inline TArray<FAnalyticsEventAttribute, Allocator> MakeAnalyticsEventAttributeArray(ArgTypes&&...Args)
{
	TArray<FAnalyticsEventAttribute, Allocator> Attrs;
	Attrs.Empty(sizeof...(Args) / 2);
	ImplMakeAnalyticsEventAttributeArray::MakeArray(Attrs, Forward<ArgTypes>(Args)...);
	return Attrs;
}

/** Helper to append to an array of attributes using a single expression. Reserves the necessary space in advance. There must be an even number of arguments, one for each key/value pair. */
template <typename Allocator = FDefaultAllocator, typename...ArgTypes>
inline TArray<FAnalyticsEventAttribute, Allocator>& AppendAnalyticsEventAttributeArray(TArray<FAnalyticsEventAttribute, Allocator>& Attrs, ArgTypes&&...Args)
{
	Attrs.Reserve(Attrs.Num() + (sizeof...(Args) / 2));
	ImplMakeAnalyticsEventAttributeArray::MakeArray(Attrs, Forward<ArgTypes>(Args)...);
	return Attrs;
}
