// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/AsyncWork.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/CriticalSection.h"
#include "HAL/Platform.h"
#include "Interfaces/IAnalyticsPropertyStore.h"
#include "Misc/DateTime.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Timespan.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "Serialization/Archive.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Stats/Stats.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"

class IFileHandle;
struct FAnalyticsEventAttribute;

/**
 * Implements a fast key/value database to store the metrics gathered to create the session summary event.
 * The store is buffered in memory until it is flushed to disk by calling Flush(). It is not designed to
 * handle millions of key/value pairs, but rather a small subset that fits in memory. The implementation
 * reserve disk space ahead of time and can usually perform in-place update. Setting or getting intrinsic
 * types (int32, uint32, int64, uint64, float, double, bool as well as FDateTime) doesn't allocate memory.
 * The string time is often updated in place if enough capacity was reserved. All store operations are
 * thread safe and atomic from the caller perspective.
 */
class FAnalyticsPropertyStore : public IAnalyticsPropertyStore
{
public:
	/** Create an empty store. */
	ANALYTICS_API FAnalyticsPropertyStore();
	ANALYTICS_API virtual ~FAnalyticsPropertyStore();

	FAnalyticsPropertyStore(const FAnalyticsPropertyStore&) = delete;
	FAnalyticsPropertyStore(FAnalyticsPropertyStore&&) = delete;
	FAnalyticsPropertyStore& operator=(const FAnalyticsPropertyStore&) = delete;
	FAnalyticsPropertyStore& operator=(FAnalyticsPropertyStore&&) = delete;

	/**
	 * Create a new store, creating the file of the specified capacity and resetting the current state.
	 * @param Pathname The path to the file to create.
	 * @param CapacityHint The desired file capacity. The implementation has a minimum and maximum capacity to reserve.
	 * @param true if the file was created and the store created, false otherwise.
	 */
	ANALYTICS_API bool Create(const FString& Pathname, uint32 CapacityHint = 4 * 1024);

	/**
	 * Load the specified file in memory, resetting the current state.
	 * @param Pathname The path to the file to open.
	 * @return true if the file was loaded successuflly, false if the file doesn't exist, the file format/version is invalid or if the file is corrupted (failed checksum).
	 */
	ANALYTICS_API bool Load(const FString& Pathname);

	/** Returns true is the store was successfully created or loaded. */
	bool IsValid() const { return FileHandle != nullptr; }

	//~ IAnalyticsPropertyStore interface
	ANALYTICS_API virtual uint32 Num() const override;
	ANALYTICS_API virtual bool Contains(const FString& Key) const override;
	ANALYTICS_API virtual bool Remove(const FString& Key) override;
	ANALYTICS_API virtual void RemoveAll() override;
	ANALYTICS_API virtual EStatusCode Set(const FString& Key, int32 Value) override;
	ANALYTICS_API virtual EStatusCode Set(const FString& Key, uint32 Value) override;
	ANALYTICS_API virtual EStatusCode Set(const FString& Key, int64 Value) override;
	ANALYTICS_API virtual EStatusCode Set(const FString& Key, uint64 Value) override;
	ANALYTICS_API virtual EStatusCode Set(const FString& Key, float Value) override;
	ANALYTICS_API virtual EStatusCode Set(const FString& Key, double Value) override;
	ANALYTICS_API virtual EStatusCode Set(const FString& Key, bool Value) override;
	ANALYTICS_API virtual EStatusCode Set(const FString& Key, const FString& Value, uint32 CharCountCapacityHint = 0) override;
	ANALYTICS_API virtual EStatusCode Set(const FString& Key, const FDateTime& Value) override;
	ANALYTICS_API virtual EStatusCode Set(const FString& Key, int32 Value, const TFunction<bool(const int32* /*Actual*/, const int32& /*Proposed*/)>& ConditionFn) override;
	ANALYTICS_API virtual EStatusCode Set(const FString& Key, uint32 Value, const TFunction<bool(const uint32* /*Actual*/, const uint32& /*Proposed*/)>& ConditionFn) override;
	ANALYTICS_API virtual EStatusCode Set(const FString& Key, int64 Value, const TFunction<bool(const int64* /*Actual*/, const int64& /*Proposed*/)>& ConditionFn) override;
	ANALYTICS_API virtual EStatusCode Set(const FString& Key, uint64 Value, const TFunction<bool(const uint64* /*Actual*/, const uint64& /*Proposed*/)>& ConditionFn) override;
	ANALYTICS_API virtual EStatusCode Set(const FString& Key, float Value, const TFunction<bool(const float* /*Actual*/, const float& /*Proposed*/)>& ConditionFn) override;
	ANALYTICS_API virtual EStatusCode Set(const FString& Key, double Value, const TFunction<bool(const double* /*Actual*/, const double& /*Proposed*/)>& ConditionFn) override;
	ANALYTICS_API virtual EStatusCode Set(const FString& Key, bool Value, const TFunction<bool(const bool* /*Actual*/, const bool& /*Proposed*/)>& ConditionFn) override;
	ANALYTICS_API virtual EStatusCode Set(const FString& Key, const FString& Value, const TFunction<bool(const FString* /*Actual*/, const FString& /*Proposed*/)>& ConditionFn) override;
	ANALYTICS_API virtual EStatusCode Set(const FString& Key, const FDateTime& Value, const TFunction<bool(const FDateTime* /*Actual*/, const FDateTime& /*Proposed*/)>& ConditionFn) override;
	ANALYTICS_API virtual EStatusCode Update(const FString& Key, const TFunction<bool(int32&)>& UpdateFn) override;
	ANALYTICS_API virtual EStatusCode Update(const FString& Key, const TFunction<bool(uint32&)>& UpdateFn) override;
	ANALYTICS_API virtual EStatusCode Update(const FString& Key, const TFunction<bool(int64&)>& UpdateFn) override;
	ANALYTICS_API virtual EStatusCode Update(const FString& Key, const TFunction<bool(uint64&)>& UpdateFn) override;
	ANALYTICS_API virtual EStatusCode Update(const FString& Key, const TFunction<bool(float&)>& UpdateFn) override;
	ANALYTICS_API virtual EStatusCode Update(const FString& Key, const TFunction<bool(double&)>& UpdateFn) override;
	ANALYTICS_API virtual EStatusCode Update(const FString& Key, const TFunction<bool(bool&)>& UpdateFn) override;
	ANALYTICS_API virtual EStatusCode Update(const FString& Key, const TFunction<bool(FString&)>& UpdateFn) override;
	ANALYTICS_API virtual EStatusCode Update(const FString& Key, const TFunction<bool(FDateTime&)>& UpdateFn) override;
	ANALYTICS_API virtual EStatusCode Get(const FString& Key, int32& OutValue) const override;
	ANALYTICS_API virtual EStatusCode Get(const FString& Key, uint32& OutValue) const override;
	ANALYTICS_API virtual EStatusCode Get(const FString& Key, int64& OutValue) const override;
	ANALYTICS_API virtual EStatusCode Get(const FString& Key, uint64& OutValue) const override;
	ANALYTICS_API virtual EStatusCode Get(const FString& Key, float& OutValue) const override;
	ANALYTICS_API virtual EStatusCode Get(const FString& Key, double& OutValue) const override;
	ANALYTICS_API virtual EStatusCode Get(const FString& Key, bool& OutValue) const override;
	ANALYTICS_API virtual EStatusCode Get(const FString& Key, FString& OutValue) const override;
	ANALYTICS_API virtual EStatusCode Get(const FString& Key, FDateTime& OutValue) const override;
	ANALYTICS_API virtual void VisitAll(const TFunction<void(FAnalyticsEventAttribute&&)>& VisitFn) const override;

	/**
	 * Flushes cached values to disk. Only one thread can flush at a time, causing possible latency if a flush is alreary happenning.
	 * @param bAsync Whether the data in flushed in a background thread or in the calling thread.
	 * @param Timeout Maximum time to wait before the flush starts (synchronous) or get scheduled (asynchronous). FTimespan::Zero() means no waiting.
	 * @return True if the flush was executed (synchronous) or scheduled (asynchronous), false if the operation timed out.
	 *
	 * @code
	 *   Store.Flush(); // Flush in the calling thread, block until all threads has finished flushing.
	 *   Store.Flush(false, FTimespan::Zero()); // Flush in the calling thread if possible, return immediatedly if another thread is already flushing.
	 *   Store.Flush(true, FTimespan::Zero()); // Flush in a backgroud thread if no other thread is flushing, return immediatedly if another flushing task is running.
	 *   Store.Flush(true);  // Flush in a background thread, might block if a thread is already flushing. (It doesn't queue several tasks).
	 * @endcode
	 */
	ANALYTICS_API virtual bool Flush(bool bAsync = false, const FTimespan& Timeout = FTimespan::MaxValue()) override;

private:
	enum class ETypeCode : uint8
	{
		I32     = 0x00,
		U32     = 0x01,
		I64     = 0x02,
		U64     = 0x03,
		Flt     = 0x04,
		Dbl     = 0x05,
		Bool    = 0x06,
		Date    = 0x07,
		Str     = 0x08,
		RawMask = 0x0F, // To extract the raw types above from the meta data below
		Dead    = 0x80, // Type for record that were moved or deleted.
	};
	FRIEND_ENUM_CLASS_FLAGS(ETypeCode)

	/** Helper task to persist the properties to disk in the background. */
	class FFlushWorker : public FNonAbandonableTask
	{
	public:
		FFlushWorker(FAnalyticsPropertyStore& InStore);
		void DoWork();
		TStatId GetStatId() const { return TStatId(); }
		static const TCHAR* Name() { return TEXT("AnalyticsPropertyStoreWorker"); }
	private:
		FAnalyticsPropertyStore& Store;
		TArray<uint8> Data;
	};
	friend class FFlushWorker;

private:
	ANALYTICS_API EStatusCode SetFixedSizeValueInternal(const FString& Key, ETypeCode TypeCode, const uint8* Value, uint32 Size);
	ANALYTICS_API EStatusCode GetFixedSizeValueInternal(const FString& Key, ETypeCode TypeCode, uint8* OutValue, uint32 Size) const;
	ANALYTICS_API EStatusCode SetStringValueInternal(const FString& Key, const FString& Value, uint32 CharCountCapacityHint = 0);
	ANALYTICS_API EStatusCode GetStringValueInternal(const FString& Key, FString& OutValue) const;
	ANALYTICS_API EStatusCode SetDateTimeValueInternal(const FString& Key, const FDateTime& Value);
	ANALYTICS_API EStatusCode GetDateTimeValueInternal(const FString& Key, FDateTime& OutValue) const;
	ANALYTICS_API void Defragment();
	bool IsFixedSize(ETypeCode Type) const { return Type != ETypeCode::Str; }
	bool IsDead(ETypeCode Type) const { return EnumHasAnyFlags(Type, ETypeCode::Dead); }
	ETypeCode RawType(ETypeCode Type) const { return Type & ETypeCode::RawMask; }
	ANALYTICS_API void FlushInternal(const TArray<uint8>& Data);
	ANALYTICS_API void Reset();

	template<typename T>
	IAnalyticsPropertyStore::EStatusCode SetFixedSizeValueInternal(const FString& Key, ETypeCode TypeCode, T Value)
	{
		return SetFixedSizeValueInternal(Key, TypeCode, reinterpret_cast<const uint8*>(&Value), sizeof(Value));
	}

	template<typename T>
	EStatusCode GetFixedSizeValueInternal(const FString& Key, ETypeCode TypeCode, T& OutValue) const
	{
		return GetFixedSizeValueInternal(Key, TypeCode, reinterpret_cast<uint8*>(&OutValue), sizeof(OutValue));
	}

private:
	mutable FTransactionallySafeCriticalSection StoreLock;
	TArray<uint8> StorageBuf;
	FMemoryWriter StorageWriter;
	mutable FMemoryReader StorageReader;
	TMap<FString, uint32> NameOffsetMap;
	TUniquePtr<IFileHandle> FileHandle;
	bool bFragmented = false;
	TUniquePtr<FAsyncTask<FFlushWorker>> FlushTask;
	TArray<uint8> AsyncFlushDataCopy;
};

ENUM_CLASS_FLAGS(FAnalyticsPropertyStore::ETypeCode);
