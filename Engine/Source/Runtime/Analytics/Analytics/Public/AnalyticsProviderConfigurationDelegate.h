// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
* Analytics providers must be configured when created.
* This delegate provides a generic way to supply this config when creating a provider from this abstract factory function.
* This delegate will be called by the specific analytics provider for each configuration value it requires.
* The first param is the confguration name, the second parameter is a bool that is true if the value is required.
* It is up to the specific provider to define what configuration keys it requires.
* Some providers may provide more typesafe ways of constructing them when the code knows exactly which provider it will be using.
*/
DECLARE_DELEGATE_RetVal_TwoParams(FString, FAnalyticsProviderConfigurationDelegate, const FString&, bool);

