// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/IAnalyticsTracer.h"

using FThreadId = uint32;

/**
* Implementation of an IAnalyticsSpan interface
*/
class FAnalyticsSpan : public IAnalyticsSpan
{
public:

	FAnalyticsSpan(FName SpanName): IAnalyticsSpan(SpanName), Name(SpanName) {};
	~FAnalyticsSpan()= default;

	// Public IAnalyticsSpan implementation
	virtual void SetProvider(TSharedPtr<IAnalyticsProvider> AnalyticsProvider) override;
	virtual void Start(const TArray<FAnalyticsEventAttribute>& AdditionalAttributes = {});
	virtual void End(const TArray<FAnalyticsEventAttribute>& AdditionalAttributes = {}) override;
	virtual void AddAttributes(const TArray<FAnalyticsEventAttribute>& AdditionalAttributes) override;
	virtual void RecordEvent(const FString& EventName, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes = {}) override;
	virtual const FName& GetName() const override;
	virtual const TArray<FAnalyticsEventAttribute>& GetAttributes() const override;
	virtual void SetStackDepth(uint32 Depth) override;
	virtual uint32 GetStackDepth() const override;
	virtual double GetElapsedTime() override;
	virtual double GetDuration() const override;
	virtual bool GetIsActive() const override;
	virtual FGuid GetId() const override;

private:

	FName								Name = "None";
	FGuid								Guid = FGuid();
	FDateTime							StartTime = 0;
	FDateTime							EndTime = 0;
	FThreadId							ThreadId = 0;
	uint32								StackDepth = 0;
	double								Duration = 0;
	bool								IsActive = false;
	TSharedPtr<IAnalyticsProvider>		AnalyticsProvider;
	TArray<FAnalyticsEventAttribute>	Attributes;;
};

/**
* Implementation of an IAnalyticsTracer interface
*/
class FAnalyticsTracer : public IAnalyticsTracer
{
public:

	FAnalyticsTracer(){};
	~FAnalyticsTracer()= default;
	
	// Public IAnalyticsTracer implementation
	virtual void StartSession() override;
	virtual void EndSession() override;
	virtual void SetProvider(TSharedPtr<IAnalyticsProvider> AnalyticsProvider) override;
	virtual TSharedPtr<IAnalyticsSpan> StartSpan(const FName Name, TSharedPtr<IAnalyticsSpan> ParentSpan, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes = {}) override;
	virtual bool EndSpan(TSharedPtr<IAnalyticsSpan> Span, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes = {}) override;
	virtual TSharedPtr<IAnalyticsSpan> GetCurrentSpan() const override;
	virtual TSharedPtr<IAnalyticsSpan> GetSessionSpan() const override;
	virtual TSharedPtr<IAnalyticsSpan> GetSpan(const FName Name) override;

private:

	bool StartSpanInternal(TSharedPtr<IAnalyticsSpan> Span, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes);
	bool EndSpanInternal(TSharedPtr<IAnalyticsSpan> Span, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes);
	TSharedPtr<IAnalyticsSpan> GetSpanInternal(const FName Name);
	void BeginRegion(TSharedPtr<IAnalyticsSpan> Span);
	void EndRegion(TSharedPtr<IAnalyticsSpan> Span);

	TSharedPtr<IAnalyticsProvider>					AnalyticsProvider;	// The Analytics provider we will send our span events to
	TSharedPtr<IAnalyticsSpan>						SessionSpan;		// The root span, this will always be present in an active session
	TArray<TSharedPtr<IAnalyticsSpan>>				ActiveSpanStack;	// Stack of active spans as WeakPtrs
	TMap<FGuid, TArray<TWeakPtr<IAnalyticsSpan>>>	SpanHeirarchy;		// List of child spans per parent span
	TMap<FName, FGuid>								RegionNames;		// Temprorary fix for duplicate span names in regions	
	FCriticalSection								CriticalSection;	
};


