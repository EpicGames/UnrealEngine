// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/DateTime.h"
#include "Templates/Function.h"
#include "AnalyticsEventAttribute.h"

/**
 * Implements a fast type safe key/value database to store analytics properties collected during a session.
 */
class IAnalyticsPropertyStore
{
public:
	/** The list of store operation outcome. */
	enum class EStatusCode : uint32
	{
		/** The operation terminated sucessfully. */
		Success,
		/** The operation failed because the key could not be found. */
		NotFound,
		/** The operation failed because the key value provided did not match the currently stored value. */
		BadType,
		/** A conditional operation failed because it was declined by the caller (during a callback). */
		Declined,
	};

public:
	virtual ~IAnalyticsPropertyStore() = default;

	/**
	 * Returns the number of elements in the store.
	 */
	virtual uint32 Num() const = 0;

	/**
	 * Returns true if the store contains the specified key.
	 */
	virtual bool Contains(const FString& Key) const = 0;

	/**
	 * Removes the specified key from the store.
	 * @return true if the key was removed.
	 */
	virtual bool Remove(const FString& Key) = 0;

	/**
	 * Removes all existing keys from the store.
	 */
	virtual void RemoveAll() = 0;

	/**
	 * Adds or updates the specified key/value pair. If the key doesn't exist, the key/value pair is added. If the key already exists, the value is updated if the stored value and the
	 * specified value types match.
	 * @param Key The property name to add or update.
	 * @param Value The property value to add or update.
	 * @return EStatusCode::Success if the key was added or updated, EStatusCode::BadType if the value type did not match the current value type.
	 */
	virtual EStatusCode Set(const FString& Key, int32 Value) = 0;
	virtual EStatusCode Set(const FString& Key, uint32 Value) = 0;
	virtual EStatusCode Set(const FString& Key, int64 Value) = 0;
	virtual EStatusCode Set(const FString& Key, uint64 Value) = 0;
	virtual EStatusCode Set(const FString& Key, float Value) = 0;
	virtual EStatusCode Set(const FString& Key, double Value) = 0;
	virtual EStatusCode Set(const FString& Key, bool Value) = 0;
	virtual EStatusCode Set(const FString& Key, const FString& Value, uint32 CharCountCapacityHint = 0) = 0;
	virtual EStatusCode Set(const FString& Key, const FDateTime& Value) = 0;

	/**
	 * Conditionnnaly adds or updates the specified key/value pair. The operation is atomic from the caller perspective. The function reads the actual value (if it exists), invoke the ConditionFn
	 * callback and write the value back if the condition function returns true. If the key doesn't exist, the actual value passed back to the condition function is null. If the condition function
	 * returns true, the proposed value is set, otherwise, the operation is declined and the store remains unchanged.
	 * @param Key The name of the property to add or update.
	 * @param Value The value proposed. This value is passed as the second parameter to the ConditionFn callback.
	 * @param ConditionFn Function invoked back with the actual value and the proposed value to let the caller decide whether the proposed value should be set or not.
	 *                    If the callback function returns true, the proposed value is added/updated, otherwise, nothing changes.
	 * @return One of the following code
	 *   - EStatusCode::Success if the entire operation succeeded.
	 *   - EStatusCode::BadType if the key/value pair exists, but the proposed value did not match the actual value type.
	 *   - EStatusCode::Declined if the callback ConditionFn returned false and the operation was declined.
	 */
	virtual EStatusCode Set(const FString& Key, int32 Value, const TFunction<bool(const int32* /*Actual*/, const int32& /*Proposed*/)>& ConditionFn) = 0;
	virtual EStatusCode Set(const FString& Key, uint32 Value, const TFunction<bool(const uint32* /*Actual*/, const uint32& /*Proposed*/)>& ConditionFn) = 0;
	virtual EStatusCode Set(const FString& Key, int64 Value, const TFunction<bool(const int64* /*Actual*/, const int64& /*Proposed*/)>& ConditionFn) = 0;
	virtual EStatusCode Set(const FString& Key, uint64 Value, const TFunction<bool(const uint64* /*Actual*/, const uint64& /*Proposed*/)>& ConditionFn) = 0;
	virtual EStatusCode Set(const FString& Key, float Value, const TFunction<bool(const float* /*Actual*/, const float& /*Proposed*/)>& ConditionFn) = 0;
	virtual EStatusCode Set(const FString& Key, double Value, const TFunction<bool(const double* /*Actual*/, const double& /*Proposed*/)>& ConditionFn) = 0;
	virtual EStatusCode Set(const FString& Key, bool Value, const TFunction<bool(const bool* /*Actual*/, const bool& /*Proposed*/)>& ConditionFn) = 0;
	virtual EStatusCode Set(const FString& Key, const FString& Value, const TFunction<bool(const FString* /*Actual*/, const FString& /*Proposed*/)>& ConditionFn) = 0;
	virtual EStatusCode Set(const FString& Key, const FDateTime& Value, const TFunction<bool(const FDateTime* /*Actual*/, const FDateTime& /*Proposed*/)>& ConditionFn) = 0;

	/**
	 * Updates an exiting value. The operation is atomic from the caller perspective. The function reads the actual value and invoke UpdateFn callback with the actual value. The
	 * callback can update the actual value or decline the operation. If the key is not found, the callback is not invoked.
	 * @param Key The name of the key to update.
	 * @param UpdateFn The function to invoke, passing the actual value.
	 * @return One of the following code
	 *   - EStatusCode::Success if the entire operation succeeded.
	 *   - EStatusCode::BadType if the key/value pair exists, but the proposed value did not match the actual value type.
	 *   - EStatusCode::Declined if the callback UpdateFn returned false and the operation was declined.
	 */
	virtual EStatusCode Update(const FString& Key, const TFunction<bool(int32& /*InOutValue*/)>& UpdateFn) = 0;
	virtual EStatusCode Update(const FString& Key, const TFunction<bool(uint32& /*InOutValue*/)>& UpdateFn) = 0;
	virtual EStatusCode Update(const FString& Key, const TFunction<bool(int64& /*InOutValue*/)>& UpdateFn) = 0;
	virtual EStatusCode Update(const FString& Key, const TFunction<bool(uint64& /*InOutValue*/)>& UpdateFn) = 0;
	virtual EStatusCode Update(const FString& Key, const TFunction<bool(float& /*InOutValue*/)>& UpdateFn) = 0;
	virtual EStatusCode Update(const FString& Key, const TFunction<bool(double& /*InOutValue*/)>& UpdateFn) = 0;
	virtual EStatusCode Update(const FString& Key, const TFunction<bool(bool& /*InOutValue*/)>& UpdateFn) = 0;
	virtual EStatusCode Update(const FString& Key, const TFunction<bool(FString& /*InOutValue*/)>& UpdateFn) = 0;
	virtual EStatusCode Update(const FString& Key, const TFunction<bool(FDateTime& /*InOutValue*/)>& UpdateFn) = 0;

	/**
	 * Reads the specified key value from the store.
	 * @param Key The name of the property to read.
	 * @param OutValue The value of the property if the function returns EStatusCode::Success, left unchanged otherwise.
	 * @return One of the following code
	 *   - EStatusCode::Success if the value was read succeessfully.
	 *   - EStatusCode::BadType if the key/value pair exists, but the type used did not match the actual value type.
	 *   - EStatusCode::NotFound if the key/value could not be found.
	 */
	virtual EStatusCode Get(const FString& Key, int32& OutValue) const = 0;
	virtual EStatusCode Get(const FString& Key, uint32& OutValue) const = 0;
	virtual EStatusCode Get(const FString& Key, int64& OutValue) const = 0;
	virtual EStatusCode Get(const FString& Key, uint64& OutValue) const = 0;
	virtual EStatusCode Get(const FString& Key, float& OutValue) const = 0;
	virtual EStatusCode Get(const FString& Key, double& OutValue) const = 0;
	virtual EStatusCode Get(const FString& Key, bool& OutValue) const = 0;
	virtual EStatusCode Get(const FString& Key, FString& OutValue) const = 0;
	virtual EStatusCode Get(const FString& Key, FDateTime& OutValue)const = 0;

	/**
	 * Iterates the keys currently stored and invokes the visitor function for each key, converting the value to its string representation.
	 * @param VisitFn The callback invoked for each key/value pair visited.
	 */
	virtual void VisitAll(const TFunction<void(FAnalyticsEventAttribute&&)>& VisitFn) const = 0;

	/**
	 * Flushes cached values to persistent storage.
	 * @param bAsync Whether the data in flushed in a background thread or in the calling thread.
	 * @param Timeout Maximum time to wait before the flush starts (synchronous) or get scheduled (asynchronous). FTimespan::Zero() means no waiting.
	 * @return True if the flush was executed (synchronous) or scheduled (asynchronous), false if the operation timed out.
	 */
	virtual bool Flush(bool bAsync = false, const FTimespan& Timeout = FTimespan::MaxValue()) = 0;
};

/**
 * Utility class to remember the key value type and let the compiler implicitly convert type or fail if such conversion is not possible.
 * Usage:
 * @code
 *   static const TAnalyticsProperty<uint32> MyProperty = TEXT("MyProperty");
 *   IAnalyticsPropertyStore* MyStore = ...;
 *   MyProperty.SetValue(MyStore,  10); // Ok.
 *   MyProperty.SetValue(MyStore, TEXT("No")); // Compile error, not convertible.
 * @endcode
 */
template<typename T>
class TAnalyticsProperty
{
public:
	TAnalyticsProperty(const TCHAR* InKey) : Key(InKey){}

	IAnalyticsPropertyStore::EStatusCode Set(IAnalyticsPropertyStore* Store, const T& Value) const { return Store->Set(Key, Value); }
	IAnalyticsPropertyStore::EStatusCode Set(IAnalyticsPropertyStore* Store, const T& Value, const TFunction<bool(const T*, const T&)>& ConditionFn) const { return Store->Set(Key, Value, ConditionFn); }
	IAnalyticsPropertyStore::EStatusCode Get(IAnalyticsPropertyStore* Store, T& OutValue) const { return Store->Get(Key, OutValue); }
	IAnalyticsPropertyStore::EStatusCode Update(IAnalyticsPropertyStore* Store, const TFunction<bool(T& InOutValue)>& UpdateFn) const { return Store->Update(Key, UpdateFn); }

	const FString Key;
};

/** Specialization for FString because the caller can specifies a capacity when adding a string. */
template<>
class TAnalyticsProperty<FString>
{
public:
	TAnalyticsProperty(const TCHAR* InKey) : Key(InKey){}

	IAnalyticsPropertyStore::EStatusCode Set(IAnalyticsPropertyStore* Store, const FString& Value, uint32 CharCountCapacityHint = 0) const { return Store->Set(Key, Value, CharCountCapacityHint); }
	IAnalyticsPropertyStore::EStatusCode Set(IAnalyticsPropertyStore* Store, const FString& Value, const TFunction<bool(const FString*, const FString&)>& ConditionFn) const { return Store->Set(Key, Value, ConditionFn); }
	IAnalyticsPropertyStore::EStatusCode Get(IAnalyticsPropertyStore* Store, FString& OutValue) const { return Store->Get(Key, OutValue); }
	IAnalyticsPropertyStore::EStatusCode Update(IAnalyticsPropertyStore* Store, const TFunction<bool(FString& InOutValue)>& UpdateFn) const { return Store->Update(Key, UpdateFn); }

	const FString Key;
};
