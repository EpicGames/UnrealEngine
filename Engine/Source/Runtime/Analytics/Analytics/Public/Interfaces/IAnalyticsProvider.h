// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnalyticsEventAttribute.h"

class Error;

/**
 * Enum describing the supported record event modes
 * @enum Cached use the existing system where the events are cached and flushed when possible
 * @enum Immediate use the new system where events are immediately flushed to a HTTP request
 */
enum class EAnalyticsRecordEventMode : uint8
{
	Cached,
	Immediate
};

const TCHAR* LexToString(EAnalyticsRecordEventMode Mode);

/** 
 * Generic interface for an analytics provider. 
 * Other modules can define more and register them with this module. See FAnalytics for details.
 *
 * Easiest way to use this is like so:
 *
 *   // MakeAnalyticsEventAttributeArray is a convenient way to efficiently make an array of attributes
 *   // Since it returns an unnamed temporary, the compiler automatically selects the move-aware version.
 *   AnalyticsProvider->RecordEvent(TEXT("MyEvent"), MakeAnalyticsEventAttributeArray("Attr1", "Value1", ... ));
 * or:
 *   TArray<FAnalyticEventAttribute> Attrs;
 *   Attrs.Add(...);
 *   AnalyticsProvider->RecordEvent(TEXT("MyEvent"), Attrs);
 * 
 * Several APIs build off the pure virtual ones. The following pure virtuals must be implemented by a derived class.
 * 
 * 	virtual bool StartSession(const TArray<FAnalyticsEventAttribute>& Attributes)
 * 	virtual void EndSession()
 * 	virtual FString GetSessionID() const
 * 	virtual bool SetSessionID(const FString& InSessionID)
 * 	virtual void FlushEvents()
 * 	virtual void SetUserID(const FString& InUserID)
 * 	virtual FString GetUserID() const
 *  virtual void RecordEvent(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes)
 *
 * There are several other methods to record specific types of events. They were added for legacy reasons to support
 * specific third party implementations. Derived classes generally do not have to worry about them.
 */
class IAnalyticsProvider
{
public:
	/**
	 * Starts a session. It's technically legal to send events without starting a session.
	 * The use case is for backends and dedicated servers to send events on behalf of a user
	 * without technically affecting the session length of the local player.
	 * Local players log in and start/end the session, but remote players simply
	 * call SetUserID and start sending events, which is legal and analytics providers should
	 * gracefully handle this.
	 * Repeated calls to this method will be ignored.
	 *
	 * @returns true if the session started successfully.
	 */
	bool StartSession()
	{
		return StartSession(TArray<FAnalyticsEventAttribute>());
	}

	/**
	 * Starts a session. See parameterless-version for contract details.
	 * @param Attributes attributes of the session. Arbitrary set of key/value pairs that will be sent
	                     with the StartSession event that this should also trigger.
	 */
	virtual bool StartSession(const TArray<FAnalyticsEventAttribute>& Attributes) = 0;

	/**
	 * Overload for StartSession that takes a single attribute
	 *
	 * @param Attribute attribute name and value
	 */
	bool StartSession(const FAnalyticsEventAttribute& Attribute)
	{
		return StartSession(TArray<FAnalyticsEventAttribute> { Attribute });
	}

	/**
	 * Overload for StartSession that takes a single name/value pair
	 *
	 * @param ParamName attribute name
	 * @param ParamValue attribute value
	 */
	bool StartSession(const FString& ParamName, const FString& ParamValue)
	{
		return StartSession(TArray<FAnalyticsEventAttribute> { FAnalyticsEventAttribute(ParamName, ParamValue) });
	}

	/**
	 * Ends the session. Usually no need to call explicitly, as the provider should do this for you when the instance is destroyed.
	 */
	virtual void EndSession() = 0;

	/**
	 * Gets the opaque session identifier string for the provider.
	 */
	virtual FString GetSessionID() const = 0;

	/**
	 * Sets the session ID of the analytics session.
	 * This is not something you normally have to do, except for 
	 * circumstances where you need to send events on behalf of another user
	 * (like a dedicated server sending events for the connected clients).
	 */
	virtual bool SetSessionID(const FString& InSessionID) = 0;

	/**
	 * Flush any cached events to the analytics provider.
	 *
	 * Note that not all providers support explicitly sending any cached events. In which case this method
	 * does nothing.
	 */
	virtual void FlushEvents() = 0;

	/**
	 * Set the UserID for use with analytics. Some providers require a unique ID
	 * to be provided when supplying events, and some providers create their own.
	 * If you are using a provider that requires you to supply the ID, use this
	 * method to set it.
	 */
	virtual void SetUserID(const FString& InUserID) = 0;

	/**
	 * Gset the current UserID.
	 * Use -ANALYTICSUSERID=<Name> command line to force the provider to use a specific UserID for this run.
	 */
	virtual FString GetUserID() const = 0;


	/**
	 * Sets an array of attributes that will automatically be appended to any event that is sent.
	 * Logical effect is like adding them to all events before calling RecordEvent.
	 * Practically, it is implemented much more efficiently from a storage and allocation perspective.
	 */
	virtual void SetDefaultEventAttributes(TArray<FAnalyticsEventAttribute>&& Attributes) = 0;

	/**
	 * @return the current array of default attributes.
	 */
	virtual TArray<FAnalyticsEventAttribute> GetDefaultEventAttributesSafe() const = 0;

	/**
	 * Used with GetDefaultAttribute to iterate over the default attributes.
	 *
	 * @return the number of default attributes are currently being applied.
	 */
	virtual int32 GetDefaultEventAttributeCount() const = 0;

	/**
	 * Used with GetDefaultEventAttributeCount to iterate over the default attributes.
	 *
	 * Range checking is not done, similar to TArray. Use GetDefaultAttributeCount() first!
	 * @return one attribute of the default attributes so we don't have to copy the entire attribute array.
	 */
	virtual FAnalyticsEventAttribute GetDefaultEventAttribute(int AttributeIndex) const = 0;

	/**
	 * Sets a user defined string as the build information/version for the session
	 */
	virtual void SetBuildInfo(const FString& InBuildInfo)
	{
		RecordEvent(TEXT("BuildInfo"), TArray<FAnalyticsEventAttribute> { FAnalyticsEventAttribute(TEXT("BuildInfo"), InBuildInfo) });
	}

	/**
	 * Sets the gender the game believes the user is as part of the session
	 */
	virtual void SetGender(const FString& InGender)
	{
		RecordEvent(TEXT("Gender"), TArray<FAnalyticsEventAttribute> { FAnalyticsEventAttribute(TEXT("Gender"), InGender) });
	}

	/**
	 * Sets the location the game believes the user is playing in as part of the session
	 */
	virtual void SetLocation(const FString& InLocation)
	{
		RecordEvent(TEXT("Location"), TArray<FAnalyticsEventAttribute> { FAnalyticsEventAttribute(TEXT("Location"), InLocation) });
	}

	/**
	 * Sets the location the game believes the user is playing in as part of the session
	 */
	virtual void SetAge(const int32 InAge)
	{
		RecordEvent(TEXT("Age"), TArray<FAnalyticsEventAttribute> { FAnalyticsEventAttribute(TEXT("Age"), InAge) });
	}

	/**
	 * Records a named event with an array of attributes
	 *
	 * @param EventName name of the event
	 * @param Attributes array of attribute name/value pairs
	 */
	virtual void RecordEvent(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes) = 0;

	/**
	 * RecordEvent overload to allow platforms to support events that are immediately sent
	 *
	 * @param EventName name of the event
	 * @param Attributes array of attribute name/value pairs
	 * @param Mode the record event mode, Cached or Immediate @see EAnalyticsRecordEventMode
	 */
	virtual void RecordEvent(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes, EAnalyticsRecordEventMode Mode)
	{
		RecordEvent(EventName, Attributes);
	}

	/**
	 * Overload for RecordEvent that takes no parameters
	 *
	 * @param EventName name of the event
	 */
	void RecordEvent(const FString& EventName)
	{
		RecordEvent(EventName, TArray<FAnalyticsEventAttribute>());
	}

	/**
	 * Overload for RecordEvent that takes a single attribute
	 *
	 * @param EventName name of the event
	 * @param Attribute attribute name and value
	 */
	void RecordEvent(const FString& EventName, const FAnalyticsEventAttribute& Attribute)
	{
		RecordEvent(EventName, TArray<FAnalyticsEventAttribute> { Attribute });
	}

	/**
	 * Overload for RecordEvent that takes a single name/value pair
	 *
	 * @param EventName name of the event
	 * @param ParamName attribute name
	 * @param ParamValue attribute value
	 */
	void RecordEvent(const FString& EventName, const FString& ParamName, const FString& ParamValue)
	{
		RecordEvent(EventName, TArray<FAnalyticsEventAttribute> { FAnalyticsEventAttribute(ParamName, ParamValue) });
	}

	/**
	 * Record an in-game purchase of a an item.
	 * 
	 * Note that not all providers support item purchase events. In this case this method
	 * is equivalent to sending a regular event with name "Item Purchase".
	 * 
	 * @param ItemId - the ID of the item, should be registered with the provider first.
	 * @param Currency - the currency of the purchase (ie, Gold, Coins, etc), should be registered with the provider first.
	 * @param PerItemCost - the cost of one item in the currency given.
	 * @param ItemQuantity - the number of Items purchased.
	 */
	virtual void RecordItemPurchase(const FString& ItemId, const FString& Currency, int PerItemCost, int ItemQuantity)
	{
		TArray<FAnalyticsEventAttribute> Params;
		Params.Add(FAnalyticsEventAttribute(TEXT("Currency"), Currency));
		Params.Add(FAnalyticsEventAttribute(TEXT("PerItemCost"), PerItemCost));
		RecordItemPurchase(ItemId, ItemQuantity, Params);
	}

	/**
	 * Record an in-game purchase of a an item.
	 * 
	 * Note that not all providers support item purchase events. In this case this method
	 * is equivalent to sending a regular event with name "Item Purchase".
	 * 
	 * @param ItemId - the ID of the item, should be registered with the provider first.
	 * @param ItemQuantity - the number of Items purchased.
	 * @param EventAttrs - a list of key/value pairs to assign to this event
	 */
	virtual void RecordItemPurchase(const FString& ItemId, int ItemQuantity, const TArray<FAnalyticsEventAttribute>& EventAttrs)
	{
		TArray<FAnalyticsEventAttribute> Params(EventAttrs);
		Params.Add(FAnalyticsEventAttribute(TEXT("ItemId"), ItemId));
		Params.Add(FAnalyticsEventAttribute(TEXT("ItemQuantity"), ItemQuantity));
		RecordEvent(TEXT("Item Purchase"), Params);
	}

	/**
	 * Record an in-game purchase of a an item.
	 * 
	 * Note that not all providers support item purchase events. In this case this method
	 * is equivalent to sending a regular event with name "Item Purchase".
	 * 
	 * @param ItemId - the ID of the item, should be registered with the provider first.
	 * @param ItemQuantity - the number of Items purchased.
	 */
	void RecordItemPurchase(const FString& ItemId, int ItemQuantity)
	{
		RecordItemPurchase(ItemId, ItemQuantity, TArray<FAnalyticsEventAttribute>());
	}

	/**
	 * Record a purchase of in-game currency using real-world money.
	 * 
	 * Note that not all providers support currency events. In this case this method
	 * is equivalent to sending a regular event with name "Currency Purchase".
	 * 
	 * @param GameCurrencyType - type of in game currency purchased, should be registered with the provider first.
	 * @param GameCurrencyAmount - amount of in game currency purchased.
	 * @param RealCurrencyType - real-world currency type (like a 3-character ISO 4217 currency code, but provider dependent).
	 * @param RealMoneyCost - cost of the currency in real world money, expressed in RealCurrencyType units.
	 * @param PaymentProvider - Provider who brokered the transaction. Generally arbitrary, but examples are PayPal, Facebook Credits, App Store, etc.
	 */
	virtual void RecordCurrencyPurchase(const FString& GameCurrencyType, int GameCurrencyAmount, const FString& RealCurrencyType, float RealMoneyCost, const FString& PaymentProvider)
	{
		TArray<FAnalyticsEventAttribute> Params;
		Params.Add(FAnalyticsEventAttribute(TEXT("RealCurrencyType"), RealCurrencyType));
		Params.Add(FAnalyticsEventAttribute(TEXT("RealMoneyCost"), RealMoneyCost));
		Params.Add(FAnalyticsEventAttribute(TEXT("PaymentProvider"), PaymentProvider));
		RecordCurrencyPurchase(GameCurrencyType, GameCurrencyAmount, Params);
	}

	/**
	 * Record a purchase of in-game currency using real-world money.
	 * 
	 * Note that not all providers support currency events. In this case this method
	 * is equivalent to sending a regular event with name "Currency Purchase".
	 * 
	 * @param GameCurrencyType - type of in game currency purchased, should be registered with the provider first.
	 * @param GameCurrencyAmount - amount of in game currency purchased.
	 * @param EventAttrs - a list of key/value pairs to assign to this event
	 */
	virtual void RecordCurrencyPurchase(const FString& GameCurrencyType, int GameCurrencyAmount, const TArray<FAnalyticsEventAttribute>& EventAttrs)
	{
		TArray<FAnalyticsEventAttribute> Params(EventAttrs);
		Params.Add(FAnalyticsEventAttribute(TEXT("GameCurrencyType"), GameCurrencyType));
		Params.Add(FAnalyticsEventAttribute(TEXT("GameCurrencyAmount"), GameCurrencyAmount));
		RecordEvent(TEXT("Currency Purchase"), Params);
	}

	/**
	 * Record an in-game purchase of a an item.
	 * 
	 * Note that not all providers support item purchase events. In this case this method
	 * is equivalent to sending a regular event with name "Item Purchase".
	 * 
	 * @param ItemId - the ID of the item, should be registered with the provider first.
	 * @param ItemQuantity - the number of Items purchased.
	 */
	virtual void RecordCurrencyPurchase(const FString& GameCurrencyType, int GameCurrencyAmount)
	{
		RecordCurrencyPurchase(GameCurrencyType, GameCurrencyAmount, TArray<FAnalyticsEventAttribute>());
	}

	/**
	 * Record a gift of in-game currency from the game itself.
	 * 
	 * Note that not all providers support currency events. In this case this method
	 * is equivalent to sending a regular event with name "Currency Given".
	 * 
	 * @param GameCurrencyType - type of in game currency given, should be registered with the provider first.
	 * @param GameCurrencyAmount - amount of in game currency given.
	 */
	virtual void RecordCurrencyGiven(const FString& GameCurrencyType, int GameCurrencyAmount)
	{
		RecordCurrencyGiven(GameCurrencyType, GameCurrencyAmount, TArray<FAnalyticsEventAttribute>());
	}

	/**
	 * Record a gift of in-game currency from the game itself.
	 * 
	 * Note that not all providers support currency events. In this case this method
	 * is equivalent to sending a regular event with name "Currency Given".
	 * 
	 * @param GameCurrencyType - type of in game currency given, should be registered with the provider first.
	 * @param GameCurrencyAmount - amount of in game currency given.
	 * @param EventAttrs - a list of key/value pairs to assign to this event
	 */
	virtual void RecordCurrencyGiven(const FString& GameCurrencyType, int GameCurrencyAmount, const TArray<FAnalyticsEventAttribute>& EventAttrs)
	{
		TArray<FAnalyticsEventAttribute> Params(EventAttrs);
		Params.Add(FAnalyticsEventAttribute(TEXT("GameCurrencyType"), GameCurrencyType));
		Params.Add(FAnalyticsEventAttribute(TEXT("GameCurrencyAmount"), GameCurrencyAmount));
		RecordEvent(TEXT("Currency Given"), Params);
	}

	/**
	 * Records an error that has happened in the game
	 * 
	 * Note that not all providers support all events. In this case this method
	 * is equivalent to sending a regular event with name "Game Error".
	 *
	 * @param Error - the error string to record
	 * @param EventAttrs - a list of key/value pairs to assign to this event
	 */
	virtual void RecordError(const FString& Error, const TArray<FAnalyticsEventAttribute>& EventAttrs)
	{
		TArray<FAnalyticsEventAttribute> Params(EventAttrs);
		Params.Add(FAnalyticsEventAttribute(TEXT("Error"), *Error));
		RecordEvent(TEXT("Game Error"), Params);
	}

	/**
	 * Records an error that has happened in the game
	 * 
	 * Note that not all providers support all events. In this case this method
	 * is equivalent to sending a regular event with name "Game Error".
	 *
	 * @param Error - the error string to record
	 */
	virtual void RecordError(const FString& Error)
	{
		RecordError(Error, TArray<FAnalyticsEventAttribute>());
	}

	/**
	 * Record a player progression event that has happened in the game
	 * 
	 * Note that not all providers support all events. In this case this method
	 * is equivalent to sending a regular event with name "Progression".
	 *
	 * @param Error - the error string to record
	 */
	virtual void RecordProgress(const FString& ProgressType, const TArray<FString>& ProgressHierarchy, const TArray<FAnalyticsEventAttribute>& EventAttrs)
	{
		TArray<FAnalyticsEventAttribute> Params(EventAttrs);
		Params.Add(FAnalyticsEventAttribute(TEXT("ProgressType"), *ProgressType));
		FString Hierarchy;
		// Build a dotted hierarchy string from the list of hierarchy progress
		for (int32 Index = 0; Index < ProgressHierarchy.Num(); Index++)
		{
			Hierarchy += ProgressHierarchy[Index];
			if (Index + 1 < ProgressHierarchy.Num())
			{
				Hierarchy += TEXT(".");
			}
		}
		Params.Add(FAnalyticsEventAttribute(TEXT("ProgressHierarchy"), *Hierarchy));
		RecordEvent(TEXT("Progression"), Params);
	}

	/**
	 * Record a player progression event that has happened in the game
	 * 
	 * Note that not all providers support all events. In this case this method
	 * is equivalent to sending a regular event with name "Progression".
	 *
	 * @param Error - the error string to record
	 */
	virtual void RecordProgress(const FString& ProgressType, const FString& ProgressHierarchy)
	{
		TArray<FString> Hierarchy;
		Hierarchy.Add(ProgressHierarchy);
		RecordProgress(ProgressType, Hierarchy, TArray<FAnalyticsEventAttribute>());
	}

	/**
	 * Record a player progression event that has happened in the game
	 * 
	 * Note that not all providers support all events. In this case this method
	 * is equivalent to sending a regular event with name "Progression".
	 *
	 * @param Error - the error string to record
	 */
	virtual void RecordProgress(const FString& ProgressType, const FString& ProgressHierarchy, const TArray<FAnalyticsEventAttribute>& EventAttrs)
	{
		TArray<FString> Hierarchy;
		Hierarchy.Add(ProgressHierarchy);
		RecordProgress(ProgressType, Hierarchy, EventAttrs);
	}

	virtual ~IAnalyticsProvider() = default;

protected:

};

