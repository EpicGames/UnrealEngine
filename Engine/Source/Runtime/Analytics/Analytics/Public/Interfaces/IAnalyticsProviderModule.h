// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "AnalyticsProviderConfigurationDelegate.h"

class IAnalyticsProvider;

/** Generic interface for an analytics provider. Other modules can define more and register them with this module. */
class IAnalyticsProviderModule : public IModuleInterface
{
public:
	virtual TSharedPtr<IAnalyticsProvider> CreateAnalyticsProvider(const FAnalyticsProviderConfigurationDelegate& GetConfigValue) const = 0;
};
