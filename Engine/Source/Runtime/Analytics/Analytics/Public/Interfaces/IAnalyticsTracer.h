// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "IAnalyticsProvider.h"

/**
 * IAnalyticsSpan Interface 
 * 
 * Inspired by OpenTelemety model for Tracer/Span.
 * A span should be used to measure long running actions. It can be thought of as hierarchical timer object.
 * A span capture event state within an array of EventAattributes and emit them with every event sent from this span
 * A span can only be started via a IAnalyticsTracer interface, but can be Ended on demand
 * Spans are arranged hierarchically and the expected behavior is for the parent span to end the spans of its children on during it's own End call
 * Elapsed time should be calculated on each call, but on when the span is active
 * Will send telemetry events via the IAnalyticsProvider
 */
class IAnalyticsSpan : FNoncopyable
{
public:

	IAnalyticsSpan(const FName Name) {};
	virtual ~IAnalyticsSpan() = default;

	/** Sets the analytics provider for the tracer */
	virtual void SetProvider(TSharedPtr<IAnalyticsProvider> AnalyticsProvider) = 0;
	
	/** Start this span */
	virtual void Start(const TArray<FAnalyticsEventAttribute>& AdditionalAttributes = {}) = 0;

	/** End this span */
	virtual void End(const TArray<FAnalyticsEventAttribute>& AdditionalAttributes = {}) = 0;

	/** Append attributes to this span context */
	virtual void AddAttributes(const TArray<FAnalyticsEventAttribute>& AdditionalAttributes) = 0;
	
	/** Record an event from this span, appends the attributes for the span context */
	virtual void RecordEvent(const FString& EventName, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes = {}) = 0;

	/** Get the internal name for the span */
	virtual const FName& GetName() const = 0;

	/** Get context attributes for the span */
	virtual const TArray<FAnalyticsEventAttribute>& GetAttributes()const = 0;

	/** Set the scope depth for the span*/
	virtual void SetStackDepth(uint32 Depth)=0;

	/** Get the scope depth for the span */
	virtual uint32 GetStackDepth() const = 0;

	/** Get the span duration in seconds */
	virtual double GetDuration() const = 0;

	/** Get the elapsed time for the span in seconds */
	virtual double GetElapsedTime() = 0;

	// Is the span active?
	virtual bool GetIsActive() const = 0;

	// Get unique Id?
	virtual FGuid GetId()  const = 0;
};

/**
 * IAnalyticsTracer Interface
 *
 * Inspired by OpenTelemety model for Tracer/Span.
 * A tracer manages and create spans. It is not permitted to create or start a span outside of the owning tracer object. 
 * By default, new spans are added as a child of the last added span but can be added to a child of a specified parent for clarity
 * The tracer has a valid SessionSpan object which is the root of the span hierarchy which is expected to be valid between the StartSession and EndSession calls.
 * Spans are hierarchical and are not expected to overlap, even though the API or implementation might well allow that.
 */
class IAnalyticsTracer : FNoncopyable
{
public:
	IAnalyticsTracer() {};
	virtual ~IAnalyticsTracer() = default;

	// Starts a new session, creates and Starts the SessionSpan object
	virtual void StartSession() =0;

	// Ends the running session and Ends the SessionSpan object
	virtual void EndSession() = 0;

	/** Sets the analytics provider for the tracer */
	virtual void SetProvider(TSharedPtr<IAnalyticsProvider> AnalyticsProvider) =0;

	/** Start a new span specifying an optional parent. EndSpan is called recursively on children. Parent attributes are passed onto children  */
	virtual TSharedPtr<IAnalyticsSpan> StartSpan(const FName Name, TSharedPtr<IAnalyticsSpan> ParentSpan=TSharedPtr<IAnalyticsSpan>(), const TArray<FAnalyticsEventAttribute>& AdditionalAttributes = {}) = 0;

	/** End an existing span*/
	virtual bool EndSpan(TSharedPtr<IAnalyticsSpan>, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes = {}) = 0;

	/** Get the currently active span */
	virtual TSharedPtr<IAnalyticsSpan> GetCurrentSpan() const = 0;

	/** Get the session span, this will always be valid for an active session */
	virtual TSharedPtr<IAnalyticsSpan> GetSessionSpan() const = 0;

	/** Get an active span by name, non active spans will not be available*/
	virtual TSharedPtr<IAnalyticsSpan> GetSpan(const FName Name) = 0;

	/** Start a new span without a parent*/
	TSharedPtr<IAnalyticsSpan> StartSpan(const FName Name, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes = {})
	{
		return StartSpan(Name, GetCurrentSpan(), AdditionalAttributes);
	};

	/** End an existing span by name*/
	bool EndSpan(const FName Name, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes = {})
	{
		return EndSpan(GetSpan(Name), AdditionalAttributes);
	}

private:
};


