// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AnalyticsCSV : ModuleRules
	{
		public AnalyticsCSV(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
						"Core"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
				"Analytics"
				}
			);
		}
	}
}
