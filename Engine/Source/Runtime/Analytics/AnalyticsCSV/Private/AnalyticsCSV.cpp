// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnalyticsCSV.h"
#include "AnalyticsProviderCSV.h"

IMPLEMENT_MODULE( FAnalyticsCSV, AnalyticsCSV );

void FAnalyticsCSV::StartupModule()
{
}

void FAnalyticsCSV::ShutdownModule()
{
}

TSharedPtr<IAnalyticsProvider> FAnalyticsCSV::CreateAnalyticsProvider(const FAnalyticsProviderConfigurationDelegate& GetConfigValue) const
{
	if (GetConfigValue.IsBound())
	{
		return MakeShared<FAnalyticsProviderCSV>(GetConfigValue);
	}

	return nullptr;
}

