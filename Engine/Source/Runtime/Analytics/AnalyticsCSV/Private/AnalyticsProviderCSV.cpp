// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnalyticsProviderCSV.h"

#include "Analytics.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Tasks/Pipe.h"

FAnalyticsProviderCSV::FAnalyticsProviderCSV(const FAnalyticsProviderConfigurationDelegate& GetConfigValue)
{
	FolderPath = GetConfigValue.Execute(TEXT("FolderPath"), true);

	if (FolderPath.IsEmpty())
	{
		// See if there's a folder specified in the environment 
		FolderPath = FPlatformMisc::GetEnvironmentVariable(TEXT("UE_TELEMETRY_DIR"));

		if (FolderPath.IsEmpty())
		{
			// Use default output path
			FolderPath = FPaths::ProjectSavedDir() / TEXT("Telemetry");
		}
	}

	if (FolderPath.IsEmpty()==false)
	{
		if (IFileManager::Get().MakeDirectory(*FolderPath, true))
		{
			WriterPipe = MakeUnique<UE::Tasks::FPipe>(TEXT("FAnalyticsProviderCSV_Writer"));
		}
	}
}

FAnalyticsProviderCSV::~FAnalyticsProviderCSV()
{
	// Make sure we close the async write pipe
	if (WriterPipe.IsValid())
	{
		WriterPipe->WaitUntilEmpty();
		WriterPipe.Reset();
	}

	// Make sure we close all the CSV file archives
	for (TArchives::TConstIterator it(Archives); it; ++it)
	{
		TSharedPtr<FArchive> Archive = (*it).Value;

		if (Archive.IsValid())
		{
			Archive->Flush();
			Archive->Close();
			Archive.Reset();
		}
	}

	Archives.Reset();
}

bool FAnalyticsProviderCSV::SetSessionID(const FString& InSessionID)
{
	SessionID = InSessionID;
	return true;
}

FString FAnalyticsProviderCSV::GetSessionID() const
{
	return SessionID;
}

void FAnalyticsProviderCSV::SetUserID(const FString& InUserID)
{
	UserID = InUserID;

}

FString FAnalyticsProviderCSV::GetUserID() const
{
	return UserID;
}

void FAnalyticsProviderCSV::FlushEvents()
{
}

void FAnalyticsProviderCSV::SetDefaultEventAttributes(TArray<FAnalyticsEventAttribute>&& Attributes)
{
	DefaultEventAttributes = Attributes;
}

TArray<FAnalyticsEventAttribute> FAnalyticsProviderCSV::GetDefaultEventAttributesSafe() const
{
	return DefaultEventAttributes;
}

int32 FAnalyticsProviderCSV::GetDefaultEventAttributeCount() const
{
	return DefaultEventAttributes.Num();
}

FAnalyticsEventAttribute FAnalyticsProviderCSV::GetDefaultEventAttribute(int AttributeIndex) const
{
	return ( AttributeIndex>=0 && AttributeIndex<DefaultEventAttributes.Num() )? DefaultEventAttributes[AttributeIndex]: FAnalyticsEventAttribute();
}

bool FAnalyticsProviderCSV::StartSession(const TArray<FAnalyticsEventAttribute>& Attributes)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAnalyticsProviderCSV::StartSession);

	RecordEvent(TEXT("StartSession"), Attributes);

	return true;
}

void FAnalyticsProviderCSV::EndSession()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAnalyticsProviderCSV::EndSession);

	RecordEvent(TEXT("EndSession"));

	// Wait until the pipe is empty before removing the archives
	if (WriterPipe.IsValid())
	{
		WriterPipe->WaitUntilEmpty();
	}

	// Make sure we close all the CSV file archives
	for (TArchives::TConstIterator it(Archives); it; ++it)
	{
		TSharedPtr<FArchive> Archive = (*it).Value;

		if (Archive.IsValid())
		{
			Archive->Flush();
			Archive->Close();
			Archive.Reset();
		}
	}

	Archives.Reset();
}

void FAnalyticsProviderCSV::RecordEvent(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes)
{	
	if (WriterPipe.IsValid() && FolderPath.IsEmpty()==false && EventName.IsEmpty()==false )
	{
		// Accumulate all the attributes together. We could have had two loops but this seems cleaner
		TArray<FAnalyticsEventAttribute> EventAttributes(Attributes);

		if (DefaultEventAttributes.Num())
		{
			// Append the defaults
			EventAttributes.Append(DefaultEventAttributes);
		}

		TSharedPtr<FArchive> Archive;

		// If we already have a registered output file for this event then use it, otherwise create a new one and add it to the archives list 
		// Ensure we have an upper limit of the maximum number of events to export
		if ( Archives.Find(EventName) == nullptr ) 
		{
			if (Archives.Num() >= MaxEvents)
			{
				// We can't create any new CSV files as we have exceeded the maximum number of allowed event files.
				return;
			}
			else
			{
				// Create the full output path
				FString FilePath = FolderPath / FPaths::MakeValidFileName(EventName) + TEXT(".csv");

				// Make a new CSV file for this event and add it to the list of files.
				Archive = TSharedPtr<FArchive>(IFileManager::Get().CreateFileWriter(*FilePath, FILEWRITE_EvenIfReadOnly));

				if (Archive.IsValid())
				{
					// Add the new;y created archive to the list
					Archives.Emplace(EventName, Archive);

					// Add the Attribute Names to the first row of the new CSV file
					TStringBuilder<1024> StringBuilder;

					int32 Column = 0;
					for (const FAnalyticsEventAttribute& Attribute : EventAttributes)
					{
						if (Column == 0)
						{
							StringBuilder.Appendf(TEXT("%s"), *Attribute.GetName());
						}
						else
						{
							StringBuilder.Appendf(TEXT(",%s"), *Attribute.GetName());
						}

						Column++;
					}

					FString OutputString = StringBuilder.ToString();

					WriterPipe->Launch(TEXT("FAnalyticsProviderCSV_WriteJob"), [Archive, OutputString = MoveTemp(OutputString)]()
						{
							Archive->Logf(TEXT("%s"), *OutputString);
							Archive->Flush();
						});
				}
			}
		}
		else
		{
			// Retrieve the existing Archive
			Archive = Archives[EventName];
		}
		
		if (Archive.IsValid())
		{
			// Append the Attribute Values to the subsequent rows of the existing CSV file
			TStringBuilder<1024> StringBuilder;

			int32 Column = 0;
			for (const FAnalyticsEventAttribute& Attribute : EventAttributes)
			{
				if (Column == 0)
				{
					StringBuilder.Appendf(TEXT("%s"), *Attribute.GetValue());
				}
				else
				{
					StringBuilder.Appendf(TEXT(",%s"), *Attribute.GetValue());
				}

				Column++;
			}

			FString OutputString = StringBuilder.ToString();

			WriterPipe->Launch(TEXT("FAnalyticsProviderCSV_WriteJob"), [Archive, OutputString = MoveTemp(OutputString)]()
				{
					Archive->Logf(TEXT("%s"), *OutputString);
					Archive->Flush();
				});

		}
	}
}
