// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnalyticsProviderConfigurationDelegate.h"
#include "Interfaces/IAnalyticsProvider.h"

namespace UE::Tasks { class FPipe; }

/**
 * Implementation of the IAnalyticsProvider interface that exports telemetry events to named CSV files
 * By default, the log files are written to Saved/Telemetry folder of the application.
 * FolderPath can be overridden in the configuration
 * For safety, there is MaxEvents number of event files that will be opened 
 */
class FAnalyticsProviderCSV : public IAnalyticsProvider
{
public:

	FAnalyticsProviderCSV(const FAnalyticsProviderConfigurationDelegate& GetConfigValue);
	~FAnalyticsProviderCSV();

	virtual bool StartSession(const TArray<FAnalyticsEventAttribute>& Attributes = {}) override;
	virtual void EndSession() override;

	virtual void FlushEvents() override;

	virtual void SetDefaultEventAttributes(TArray<FAnalyticsEventAttribute>&& Attributes)  override;
	virtual TArray<FAnalyticsEventAttribute> GetDefaultEventAttributesSafe() const override;
	virtual int32 GetDefaultEventAttributeCount() const  override;
	virtual FAnalyticsEventAttribute GetDefaultEventAttribute(int AttributeIndex) const  override;
	virtual bool SetSessionID(const FString& InSessionID) override;
	virtual void SetUserID(const FString& InUserID) override;

	virtual FString GetSessionID() const override;
	virtual FString GetUserID() const override;
	
	virtual void RecordEvent(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes = {}) override;

private:

	using TArchives = TMap < FString, TSharedPtr<FArchive>>;

	FString									UserID;
	FString									SessionID;
	FString									FolderPath;
	TArray<FAnalyticsEventAttribute>		DefaultEventAttributes;	
	int32									MaxEvents = 16;
	TUniquePtr<UE::Tasks::FPipe>			WriterPipe;
	TArchives								Archives;
	bool									SessionStarted = false;
	
};
