// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnalyticsProviderConfigurationDelegate.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Interfaces/IAnalyticsProviderModule.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

class IAnalyticsProvider;

/**
 *  Public implementation of IAnalyticsProviderModule that returns a FAnalyticsProviderCSV object
 */
class FAnalyticsCSV : public IAnalyticsProviderModule
{
	//--------------------------------------------------------------------------
	// Module functionality
	//--------------------------------------------------------------------------
public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline FAnalyticsCSV& Get()
	{
		return FModuleManager::LoadModuleChecked< FAnalyticsCSV >( "AnalyticsCSV" );
	}

	/**
	 * IAnalyticsProviderModule interface.
	 */
	ANALYTICSCSV_API virtual TSharedPtr<IAnalyticsProvider> CreateAnalyticsProvider(const FAnalyticsProviderConfigurationDelegate& GetConfigValue) const override;

private:
	ANALYTICSCSV_API virtual void StartupModule() override;
	ANALYTICSCSV_API virtual void ShutdownModule() override;
};

