// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AnalyticsET : ModuleRules
	{
		public AnalyticsET(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
				}
				);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Analytics",
					"HTTP",
					"Json",
                    // ... add private dependencies that you statically link with here ...
				}
				);

			PublicIncludePathModuleNames.Add("Analytics");
			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
		}
	}
}
