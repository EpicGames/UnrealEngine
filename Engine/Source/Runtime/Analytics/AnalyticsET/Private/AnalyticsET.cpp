// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnalyticsET.h"
#include "IAnalyticsProviderET.h"

#include "HttpModule.h"
#include "Analytics.h"
#include "AnalyticsPerfTracker.h"
#include "AnalyticsProviderETEventCache.h"
#include "Misc/EngineVersion.h"

IMPLEMENT_MODULE( FAnalyticsET, AnalyticsET );

void FAnalyticsET::StartupModule()
{
	// Make sure http is loaded so that we can flush events during module shutdown
	FModuleManager::LoadModuleChecked<FHttpModule>("HTTP");
	FAnalyticsProviderETEventCache::OnStartupModule();
}

void FAnalyticsET::ShutdownModule()
{
#if ANALYTICS_PERF_TRACKING_ENABLED
	TearDownAnalyticsPerfTracker();
#endif
}

TSharedPtr<IAnalyticsProvider> FAnalyticsET::CreateAnalyticsProvider(const FAnalyticsProviderConfigurationDelegate& GetConfigValue) const
{
	return CreateAnalyticsProviderET(GetConfigValue);
}

TSharedPtr<IAnalyticsProviderET> FAnalyticsET::CreateAnalyticsProviderET(const FAnalyticsProviderConfigurationDelegate& GetConfigValue) const
{
#ifdef DISABLE_ANALYTICS_PROVIDER 
	UE_LOGF(LogAnalytics, Warning, "CreateAnalyticsProvider is disabled in this configuration");
	return NULL;
#else
	if (GetConfigValue.IsBound())
	{
		Config ConfigValues;
		ConfigValues.APIKeyET = GetConfigValue.Execute(Config::GetKeyNameForAPIKey(), true);
		ConfigValues.APIServerET = GetConfigValue.Execute(Config::GetKeyNameForAPIServer(), true);
		ConfigValues.APIEndpointET = GetConfigValue.Execute(Config::GetKeyNameForAPIEndpoint(), true);
		ConfigValues.AppVersionET = GetConfigValue.Execute(Config::GetKeyNameForAppVersion(), false);
		ConfigValues.UseLegacyProtocol = FCString::ToBool(*GetConfigValue.Execute(Config::GetKeyNameForUseLegacyProtocol(), false));
		ConfigValues.APIUnixSocketPathET = GetConfigValue.Execute(Config::GetKeyNameForAPIUnixSocketPath(), false);

		if (ConfigValues.AppVersionET.IsEmpty())
		{
			ConfigValues.AppVersionET = FEngineVersion::Current().ToString();
		}

		if (!ConfigValues.UseLegacyProtocol)
		{
			ConfigValues.AppEnvironment = GetConfigValue.Execute(Config::GetKeyNameForAppEnvironment(), true);
			ConfigValues.UploadType = GetConfigValue.Execute(Config::GetKeyNameForUploadType(), true);
		}
		return CreateAnalyticsProvider(ConfigValues);
	}
	else
	{
		UE_LOGF(LogAnalytics, Warning, "CreateAnalyticsProvider called with an unbound delegate");
	}
	return NULL;
#endif
}
