// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnalyticsFlowTracker.h"
#include "ProfilingDebugging/MiscTrace.h"

void FAnalyticsFlowTracker::SetProvider(TSharedPtr<IAnalyticsProvider> InProvider)
{
	AnalyticsProvider = InProvider;
}

void FAnalyticsFlowTracker::StartSession()
{
}

void FAnalyticsFlowTracker::EndSession()
{
	FScopeLock ScopeLock(&CriticalSection);

	// End all the open flows from the stack
	while (FlowDataStack.Num())
	{
		// We are forcibly shutting down the sub flows so mark them as unsuccesful
		EndFlowInternal(FlowDataStack.Last(), false);
	}

	ensure(FlowDataRegistry.IsEmpty());
	ensure(FlowGuidRegistry.IsEmpty());

	AnalyticsProvider.Reset();
}

FGuid FAnalyticsFlowTracker::StartFlow(const FName& NewFlowName)
{
	FScopeLock ScopeLock(&CriticalSection);
	TRACE_BEGIN_REGION(*NewFlowName.ToString());

	// Create a new Guid for this flow, can we assume it is unique?
	FGuid NewFlowGuid = FGuid::NewGuid();
	ensureMsgf(FlowDataRegistry.Find(NewFlowGuid) == nullptr, TEXT("Could not generate a unique flow guid."));

	FFlowData FlowData;

	FlowData.StartTime = FDateTime::UtcNow();
	FlowData.FlowName = NewFlowName;
	FlowData.FlowGuid = NewFlowGuid;
	FlowData.ThreadId = FPlatformTLS::GetCurrentThreadId();

	// Register the name and guid pair
	FlowGuidRegistry.Add(NewFlowName, NewFlowGuid);
	FlowDataRegistry.Add(NewFlowGuid, FlowData);
	FlowDataStack.Add(NewFlowGuid);

	return NewFlowGuid;
}

FGuid FAnalyticsFlowTracker::StartSubFlow(const FName& NewSubFlowName)
{
	FScopeLock ScopeLock(&CriticalSection);
	
	if (FlowDataStack.Num()>0)
	{
		return StartSubFlowInternal(NewSubFlowName, FlowDataStack.Last(0));
	}

	return FGuid();
}

FGuid FAnalyticsFlowTracker::StartSubFlow(const FName& NewSubFlowName, const FGuid& FlowGuid)
{
	FScopeLock ScopeLock(&CriticalSection);
	return StartSubFlowInternal(NewSubFlowName, FlowGuid);
}

FGuid FAnalyticsFlowTracker::StartSubFlowInternal(const FName& NewSubFlowName, const FGuid& FlowGuid)
{
	FFlowData* FlowData = FlowDataRegistry.Find(FlowGuid);

	if (ensureMsgf(FlowData, TEXT("SubFlow started outside of a valid flow scope")))
	{
		TRACE_BEGIN_REGION(*NewSubFlowName.ToString());

		// Create a new Guid for this SubFlow, can we assume it is unique?
		FGuid NewSubFlowGuid = FGuid::NewGuid();
		ensureMsgf(SubFlowDataRegistry.Find(NewSubFlowGuid) == nullptr, TEXT("Could not generate a unique SubFlow guid."));

		// Register the name and guid pair
		SubFlowGuidRegistry.Add(NewSubFlowName, NewSubFlowGuid);

		FSubFlowData NewSubFlow;

		NewSubFlow.SubFlowGuid = NewSubFlowGuid;
		NewSubFlow.SubFlowName = NewSubFlowName;
		NewSubFlow.StartTime = FDateTime::UtcNow();
		NewSubFlow.EndTime = 0;
		NewSubFlow.ScopeDepth = FlowData->SubFlowDataStack.Num();
		NewSubFlow.ThreadId = FPlatformTLS::GetCurrentThreadId();

		// Add SubFlow to Flow
		NewSubFlow.FlowGuid = FlowData->FlowGuid;
		NewSubFlow.FlowName = FlowData->FlowName;
		FlowData->SubFlowDataArray.Add(NewSubFlowGuid);
		FlowData->SubFlowDataStack.Add(NewSubFlowGuid);

		// Register the name and guid pair
		SubFlowGuidRegistry.Add(NewSubFlowName, NewSubFlowGuid);
		SubFlowDataRegistry.Add(NewSubFlowGuid, NewSubFlow);

		return NewSubFlowGuid;
	}

	return FGuid();
}

bool FAnalyticsFlowTracker::EndSubFlowInternal(const FGuid& SubFlowGuid, bool bSuccess, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes)
{
	if (SubFlowGuid.IsValid() == false)
		return false;

	FSubFlowData* SubFlowData = SubFlowDataRegistry.Find(SubFlowGuid);

	if (SubFlowData == nullptr)
		return false;

	const FGuid FlowGuid = SubFlowData->FlowGuid;

	if (SubFlowData->EndTime<SubFlowData->StartTime)
	{
		// Don't record again if it has already ended
		SubFlowData->EndTime = FDateTime::UtcNow();
		SubFlowData->bSuccess = bSuccess;

		const FTimespan TimeTaken = SubFlowData->EndTime - SubFlowData->StartTime;
		SubFlowData->TimeInSeconds = TimeTaken.GetTotalSeconds();
		SubFlowData->AdditionalEventAttributes = AdditionalAttributes;

		TArray<FAnalyticsEventAttribute> EventAttributes = AdditionalAttributes;

		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("SchemaVersion"), SubFlowSchemaVersion));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("SubFlowGUID"), *SubFlowData->SubFlowGuid.ToString()));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("SubFlowName"), *SubFlowData->SubFlowName.ToString()));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("FlowGUID"), SubFlowData->FlowGuid.ToString()));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("FlowName"), *SubFlowData->FlowName.ToString()));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("ThreadId"), SubFlowData->ThreadId));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("StartUTC"), SubFlowData->StartTime.ToUnixTimestampDecimal()));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("TimeInSec"), SubFlowData->TimeInSeconds));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("Success"), SubFlowData->bSuccess));

		if (AnalyticsProvider.IsValid())
		{
			AnalyticsProvider->RecordEvent(SubFlowEventName, EventAttributes);
		}

		TRACE_END_REGION(*SubFlowData->SubFlowName.ToString());

		FFlowData* FlowData = FlowDataRegistry.Find(FlowGuid);

		if (ensureMsgf(FlowData, TEXT("A sub flow does not belong to a valid flow.")))
		{
			// Most likely it will be the ending item
			for (int32 index = FlowData->SubFlowDataStack.Num() - 1; index >= 0; index--)
			{
				if (FlowData->SubFlowDataStack[index] == SubFlowGuid)
				{
					// Remove the sub flow from the stack
					FlowData->SubFlowDataStack.RemoveAt(index);
					break;
				}
			}
		}
	}
	
	return true;
}

bool FAnalyticsFlowTracker::EndSubFlow(const FGuid& SubFlowGuid, bool bSuccess, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes)
{
	FScopeLock ScopeLock(&CriticalSection);
	if (SubFlowGuid.IsValid())
	{
		return EndSubFlowInternal(SubFlowGuid, bSuccess, AdditionalAttributes);
	}

	return false;
}

bool FAnalyticsFlowTracker::EndSubFlow(const FName& SubFlowName, bool bSuccess, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes)
{
	FScopeLock ScopeLock(&CriticalSection);
	FGuid* SubFlowGuid = SubFlowGuidRegistry.Find(SubFlowName);
	if (SubFlowGuid)
	{
		return EndSubFlowInternal(*SubFlowGuid, bSuccess, AdditionalAttributes);
	}

	return false;
}

static void AggregateAttributes(TArray<FAnalyticsEventAttribute>& AggregatedAttibutes, const TArray<FAnalyticsEventAttribute>& Attributes)
{
	// Aggregates all attributes
	for (const FAnalyticsEventAttribute& Attribute : Attributes)
	{
		bool AttributeWasFound=false;

		for (FAnalyticsEventAttribute& AggregatedAttribute : AggregatedAttibutes )
		{
			if (Attribute.GetName() == AggregatedAttribute.GetName())
			{
				AggregatedAttribute += Attribute;
				
				// If we already have this attribute then great no more to do for this attribute
				AttributeWasFound = true;
				break;
			}
		}

		if (AttributeWasFound == false)
		{
			// No matching attribute so append
			AggregatedAttibutes.Add(Attribute);
		}	
	}
}

bool FAnalyticsFlowTracker::EndFlow(const FName& FlowName, bool bSuccess, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes)
{
	FScopeLock ScopeLock(&CriticalSection);
	if (FGuid* FlowGuid = FlowGuidRegistry.Find(FlowName))
	{
		return EndFlowInternal(*FlowGuid, bSuccess, AdditionalAttributes);
	}

	return false;
}

bool FAnalyticsFlowTracker::EndFlow(bool bSuccess, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes)
{
	FScopeLock ScopeLock(&CriticalSection);

	if (FlowDataStack.IsEmpty() == false)
	{
		return EndFlowInternal(FlowDataStack.Last(0), bSuccess, AdditionalAttributes);
	}

	return false;
}

bool FAnalyticsFlowTracker::EndFlow(const FGuid& FlowGuid, bool bSuccess, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes)
{
	FScopeLock ScopeLock(&CriticalSection);
	return EndFlowInternal(FlowGuid, bSuccess, AdditionalAttributes);
}

bool FAnalyticsFlowTracker::EndFlowInternal(const FGuid& FlowGuid, bool bSuccess, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes)
{
	if (FlowGuid.IsValid() == false)
		return false;

	FFlowData* FlowData = FlowDataRegistry.Find(FlowGuid);
	
	if (FlowData == nullptr)
		return false;

	FlowData->EndTime = FDateTime::UtcNow();
	const FTimespan WallTime = FlowData->EndTime - FlowData->StartTime;
	FlowData->TimeInSeconds = WallTime.GetTotalSeconds();

	TArray<FAnalyticsEventAttribute> EventAttributes = AdditionalAttributes;
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("SchemaVersion"), FlowSchemaVersion));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("FlowGUID"), FlowData->FlowGuid.ToString()));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("FlowName"), FlowData->FlowName.ToString()));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("ThreadId"), FlowData->ThreadId));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("StartUTC"), FlowData->StartTime.ToUnixTimestampDecimal()));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("Success"), bSuccess));
	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("WallTimeInSec"), FlowData->TimeInSeconds));

	double TotalTimeInSeconds = 0;

	for (FGuid SubFlowGuid : FlowData->SubFlowDataArray)
	{
		EndSubFlowInternal(SubFlowGuid);

		FSubFlowData* SubFlowData = SubFlowDataRegistry.Find(SubFlowGuid);

		if (ensureMsgf(SubFlowData, TEXT("SubFlow does not exist.")))
		{
			// Aggregate the additional attributes from the sub flows
			AggregateAttributes(EventAttributes, SubFlowData->AdditionalEventAttributes);
			TotalTimeInSeconds += SubFlowData->TimeInSeconds;
		}
	}

	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("TotalTimeInSec"), TotalTimeInSeconds));

	if (AnalyticsProvider.IsValid())
	{
		AnalyticsProvider->RecordEvent(FlowEventName, EventAttributes);
		AnalyticsProvider->FlushEvents();
	}

	TRACE_END_REGION(*FlowData->FlowName.ToString());

	// Clear up our data
	for (FGuid SubFlowGuid : FlowData->SubFlowDataArray)
	{
		FSubFlowData* SubFlowData = SubFlowDataRegistry.Find(SubFlowGuid);

		if (ensureMsgf(SubFlowData, TEXT("SubFlow does not exist.")))
		{
			// Remove the SubFlow and guid from the registry
			SubFlowDataRegistry.Remove(SubFlowGuid);
			SubFlowGuidRegistry.Remove(SubFlowData->SubFlowName);
		}
	}

	// Remove the flow and guid from the registry
	FlowDataRegistry.Remove(FlowData->FlowGuid);
	FlowGuidRegistry.Remove(FlowData->FlowName);

	// Remove the FlowData from the stack
	for (int32 index = FlowDataStack.Num() - 1; index >= 0; index--)
	{
		if (FlowDataStack[index] == FlowGuid)
		{
			// Remove the flow from the stack
			FlowDataStack.RemoveAt(index);
			break;
		}
	}

	return true;
}
