// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnalyticsPerfTracker.h"

#include "Misc/CommandLine.h"
#include "Stats/Stats.h"
#include "Misc/EngineVersion.h"

#if ANALYTICS_PERF_TRACKING_ENABLED

FAnalyticsPerfTracker::FAnalyticsPerfTracker()
{
	bEnabled = FParse::Param(FCommandLine::Get(), TEXT("ANALYTICSTRACKPERF"));
	if (bEnabled)
	{
		LogFile.SetSuppressEventTag(true);
		LogFile.Serialize(TEXT("Date,CL,RunID,Time,WindowSeconds,ProfiledSeconds,Frames,Flushes,Events,Bytes,FrameCounter"), ELogVerbosity::Log, FName());
		LastSubmitTime = StartTime;
		StartDate = FDateTime::UtcNow().ToIso8601();
		CL = LexToString(FEngineVersion::Current().GetChangelist());
	}
}

void FAnalyticsPerfTracker::RecordFlush(uint64 Bytes, uint64 NumEvents, double TimeSec)
{
	if (bEnabled)
	{
		++FlushesThisWindow;
		BytesThisWindow += Bytes;
		NumEventsThisWindow += NumEvents;
		TimeThisWindow += TimeSec;
	}
}

bool FAnalyticsPerfTracker::IsEnabled() const
{
	return bEnabled;
}

void FAnalyticsPerfTracker::SetRunID(const FString& InRunID)
{
	if (bEnabled)
	{
		RunID = InRunID;
		StartDate = FDateTime::UtcNow().ToIso8601();
	}
}

bool FAnalyticsPerfTracker::Tick(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_IAnalyticsProviderET_Tick);

	if (bEnabled)
	{
		++FramesThisWindow;
		double Now = FPlatformTime::Seconds();
		if (WindowExpired(Now))
		{
			LogFile.Serialize(*FString::Printf(TEXT("%s,%s,%s,%f,%f,%f,%d,%d,%" UINT64_FMT ",%" UINT64_FMT ",%" UINT64_FMT),
				*StartDate,
				*CL,
				*RunID,
				Now - StartTime,
				Now - LastSubmitTime,
				TimeThisWindow,
				FramesThisWindow,
				FlushesThisWindow,
				NumEventsThisWindow,
				BytesThisWindow,
				(uint64)GFrameCounter),
				ELogVerbosity::Log, FName(), Now);
			ResetWindow(Now);
		}
		return true;
	}
	else
	{
		return false;
	}
}

bool FAnalyticsPerfTracker::WindowExpired(double Now)
{
	return Now > LastSubmitTime + 60.0;
}

void FAnalyticsPerfTracker::ResetWindow(double Now)
{
	LastSubmitTime = Now;
	TimeThisWindow = 0.0;
	BytesThisWindow = 0;
	NumEventsThisWindow = 0;
	FlushesThisWindow = 0;
	FramesThisWindow = 0;
}

ANALYTICSET_API void SetAnayticsETPerfTrackingRunID(const FString& RunID)
{
	GetAnalyticsPerfTracker().SetRunID(RunID);
}

ANALYTICSET_API FAnalyticsPerfTracker& GetAnalyticsPerfTracker()
{
	return TLazySingleton<FAnalyticsPerfTracker>::Get();
}

ANALYTICSET_API void TearDownAnalyticsPerfTracker()
{
	TLazySingleton<FAnalyticsPerfTracker>::TearDown();
}

#endif // #if ANALYTICS_PERF_TRACKING_ENABLED
