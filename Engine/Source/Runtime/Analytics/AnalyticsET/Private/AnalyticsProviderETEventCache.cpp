// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnalyticsProviderETEventCache.h"
#include "IAnalyticsProviderET.h"
#include "Analytics.h"
#include "Misc/ScopeLock.h"
#include "PlatformHttp.h"
#include "Algo/Accumulate.h"
#include "Serialization/JsonWriter.h"
#include "Containers/StringConv.h"
#include "Misc/StringBuilder.h"
#include "Misc/CString.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformTime.h"

namespace EventCacheStatic
{
	static float PayloadPercentageOfMaxForWarning = 1.00f;
	FAutoConsoleVariableRef CvarPayloadPercentageOfMaxForWarning(
		TEXT("AnalyticsET.PayloadPercentageOfMaxForWarning"),
		PayloadPercentageOfMaxForWarning,
		TEXT("Percentage of the maximum payload for an EventCache that will trigger a warning message, listing the events in the payload. This is intended to be used to investigate spammy or slow telemetry.")
	);

	static float PayloadFlushTimeSecForWarning = 0.001f;
	FAutoConsoleVariableRef CvarPayloadFlushTimeSecForWarning(
		TEXT("AnalyticsET.PayloadFlushTimeSecForWarning"),
		PayloadFlushTimeSecForWarning,
		TEXT("Time in seconds that flushing an EventCache payload can take before it will trigger a warning message, listing the events in the payload. This is intended to be used to investigate spammy or slow telemetry.")
	);

	/** Used for testing below to ensure stable output */
	bool bUseZeroDateOffset = false;

	inline int ComputeAttributeSize(const FAnalyticsEventAttribute& Attribute)
	{
		return 
		// "              Name             "   :             Value              ,   (maybequoted)                                          
		   1 + Attribute.GetName().Len() + 1 + 1 + Attribute.GetValue().Len() + 1 + (Attribute.IsJsonFragment() ? 0 : 2);
	}

	inline int ComputeAttributeSize(const TArray<FAnalyticsEventAttribute>& Attributes)
	{
		return Algo::Accumulate(Attributes, 0, [](int Accum, const FAnalyticsEventAttribute& Attr) { return Accum + EventCacheStatic::ComputeAttributeSize(Attr); });
	}

	inline int ComputeEventSize(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes, int CurrentDefaultAttributeSizeEstimate)
	{
		return
			// "{EventName":"   EVENT_NAME     ",
					 14 +   EventName.Len() + 2
			// "DateOffset":"+00:00:00.000",
						 + 29
			// ATTRIBUTES_SIZE
			+ CurrentDefaultAttributeSizeEstimate
			// ATTRIBUTES_SIZE
			+ ComputeAttributeSize(Attributes)
			// Last attribute will not have a comma, so subtract that off the estimate.
			- 1
			// "},"
			+2
			;

	}

	// We need to allocate some stack space (inline storage) for UTF8 conversion strings. This is the longest attribute value we can support without imposing a dynamic allocation
	constexpr int32 ConversionBufferSize = 512;
	// This is the buffer we will convert strings into UTF8 into, since it's difficult to convert them directly into a TArray<>, since it doesn't know how to resize itself.
	// We also don't want to walk the string once to count the chars if we don't have to. so we pay the price to copy directly into a stack-allocated buffer most of the time,
	// but let it spill over to a dynamic allocation for long strings.
	typedef TStringBuilder<ConversionBufferSize> FJsonStringBuilder;

	const ANSICHAR* PayloadTemplate = "{\"Events\":[]}";
	const int32 PayloadTemplateLength = 13;
	const ANSICHAR* PayloadTrailer = "]}";
	const int32 PayloadTrailerLength = 2;

	/** Appends UTF8 chars directly to a UTF8 stream. Must already be properly UTF8 encoded. Does NOT add a NULL terminator. */
	inline void AppendString(TArray<uint8>& UTF8Stream, const ANSICHAR* UTF8Chars, int32 CharCount)
	{
		UTF8Stream.Append(reinterpret_cast<const uint8*>(UTF8Chars), CharCount);
	}

	/** 
	 * Appends a TCHAR* string (need not be null-terminated) to a UTF8 stream.
	 * Converts the string directly into the UTF8Stream. Does NOT add a NULL terminator. 
	 * 
	 * This function is highly optimized for efficiency. writes directly into the output stream without precomputing the string length.
	 * Optimistically adds a bit of space to handle ocassional multibyte chars, but keeps growing until it fits.
	 * In practice, this makes this function 30-40% faster than precomputing the string length in advance,
	 * and over 2x faster than usig FStringConversion<> directly, even with an appropriately sized buffer.
	 */
	inline void AppendString(TArray<uint8>& UTF8Stream, const TCHAR* Str, int32 Len)
	{
		const int32 OldLen = UTF8Stream.Num();

		// *** ORIGINAL, simpler code. But slower. ***
		// convert directly into new array, precompute length
		// get the string length and expand our buffer to fit it.
		//const int32 StrLen = FPlatformString::ConvertedLength<UTF8CHAR>(Str, Len);
		//UTF8Stream.SetNumUninitialized(OldLen + StrLen, EAllowShrinking::No);
		//FPlatformString::Convert((UTF8CHAR*)&UTF8Stream[OldLen], StrLen, Str, Len);

		// optimistically allocate a bit of extra space and see if we fill up the buffer.
		// If we do, lengthen the buffer a bit and try again.
		// This works 33% better than always precomputing the string length in practice, as walking over the chars to find the actual length is pretty slow.
		bool bWroteFullString = false;
		float SizeMultiplier = 0.25f;
		while (!bWroteFullString)
		{
			// Give some padding. ensure we add at least one char.
			const int32 StrLen = Len + (int32)FMath::Max(1.f, (float)Len * SizeMultiplier);
			// make space for the string
			UTF8Stream.SetNumUninitialized(OldLen + StrLen, EAllowShrinking::No);
			// convert it to UTF8
			if (UTF8CHAR* NewEnd = FPlatformString::Convert((UTF8CHAR*)&UTF8Stream[OldLen], StrLen, Str, Len))
			{
				// truncate to that length.
				UTF8Stream.SetNum(OldLen + (int32)(NewEnd - (UTF8CHAR*)&UTF8Stream[OldLen]), EAllowShrinking::No);
				bWroteFullString = true;
			}
			else
			{
				// we overflowed our buffer. Must be lots of multibyte chars. double the slack and try again.
				SizeMultiplier *= 2.0f;
				// if we grow too much, give up and compute the true chars needed.
				if (SizeMultiplier >= 2.0f)
				{
					const int32 ActualCharsNeeded = FPlatformString::ConvertedLength<UTF8CHAR>(Str, Len);
					UTF8Stream.SetNumUninitialized(OldLen + ActualCharsNeeded, EAllowShrinking::No);
					// convert it to UTF8 using the known number of charts
					FPlatformString::Convert((UTF8CHAR*)&UTF8Stream[OldLen], ActualCharsNeeded, Str, Len);
					bWroteFullString = true;
				}
			}
		}
	}

	/** Appends an FString efficiently into a UTF8 stream. Does NOT add a NULL terminator. */
	inline void AppendString(TArray<uint8>& UTF8Stream, const FString& Str)
	{
		AppendString(UTF8Stream, *Str, Str.Len());
	}

	/** Appends an TStringBuilder efficiently into a UTF8 stream. Does NOT add a NULL terminator. */
	inline void AppendString(TArray<uint8>& UTF8Stream, const FJsonStringBuilder& str)
	{
		AppendString(UTF8Stream, str.GetData(), str.Len());
	}

	/** Append a Json string to a UTF8 stream. Escapes the string, adds quotes, and converts it to UTF8 in temp space. Does NOT add a NULL terminator. If it's a JsonFragment, doesn't escape or add the quotes. */
	inline void AppendJsonString(TArray<uint8>& UTF8Stream, FJsonStringBuilder& JsonStringBuilder, const FString& str, bool bIsJsonFragment)
	{
		if (bIsJsonFragment)
		{
			// if it's a JsonFragment, not need to escape. Write it straight out.
			AppendString(UTF8Stream, str);
		}
		else
		{
			// always reset first.
			JsonStringBuilder.Reset();
			// escape the Json and add quotes
			AppendEscapeJsonString(JsonStringBuilder, str);
			// Add "<NAME>"
			AppendString(UTF8Stream, JsonStringBuilder);

		}
	}

	/** Append an AnalyticsEventAttribute to a UTF8 stream: ,"<NAME>":<VALUE> */
	inline void AppendEventAttribute(TArray<uint8>& UTF8Stream, FJsonStringBuilder& JsonStringBuilder, const FAnalyticsEventAttribute& Attr)
	{
		// Add ,
		UTF8Stream.Add(static_cast<uint8>(','));
		AppendJsonString(UTF8Stream, JsonStringBuilder, Attr.GetName(), false);
		// Add :
		UTF8Stream.Add(static_cast<uint8>(':'));
		AppendJsonString(UTF8Stream, JsonStringBuilder, Attr.GetValue(), Attr.IsJsonFragment());
	}

	inline void InitializePayloadBuffer(TArray<uint8>& Buffer, int32 MaximumPayloadSize)
	{
		Buffer.Reserve((int32)(MaximumPayloadSize * 1.2));
		// we are going to write UTF8 directly into our payload buffer.
		AppendString(Buffer, PayloadTemplate, PayloadTemplateLength);
	}
}

ANALYTICSET_API void FAnalyticsProviderETEventCache::OnStartupModule()
{
}

FAnalyticsProviderETEventCache::FAnalyticsProviderETEventCache(int32 InMaximumPayloadSize, int32 InPreallocatedPayloadSize)
: MaximumPayloadSize(InMaximumPayloadSize)
, PreallocatedPayloadSize(InPreallocatedPayloadSize)
{
	// reserve space for a few flushes to build up.
	FlushQueue.Reserve(4);
	// reserve space for a few entries to build up.
	CachedEventEntries.Reserve(100);

	if (MaximumPayloadSize < 0)
	{
		// default to 100KB.
		MaximumPayloadSize = 100*1024;
		GConfig->GetInt(TEXT("AnalyticsProviderETEventCache"), TEXT("MaximumPayloadSize"), MaximumPayloadSize, GEngineIni);
	}

	if (PreallocatedPayloadSize < 0)
	{
		PreallocatedPayloadSize = MaximumPayloadSize;
	}
	// allocate the payload buffer to the maximum size, and insert the payload template to start with.
	EventCacheStatic::InitializePayloadBuffer(CachedEventUTF8Stream, PreallocatedPayloadSize);
}

// We start with {"Events":[]}
// We End with {"Events":[{"EventName":"<NAME>","DateOffset":"<OFFSET>",<DefaultAttrs>,<Attrs>}]}
void FAnalyticsProviderETEventCache::AddToCache(FString EventName, const TArray<FAnalyticsEventAttribute>& Attributes)
{
	UE::TScopeLock ScopedLock(CachedEventsCS);

	// If we estimate that 110% of the size estimate (in case there are a lot of Json escaping or multi-byte UTF8 chars) will exceed our max payload, queue up a flush. 
	const int32 EventSizeEstimate = EventCacheStatic::ComputeEventSize(EventName, Attributes, CachedDefaultAttributeUTF8Stream.Num());
	if (CachedEventUTF8Stream.Num() + (EventSizeEstimate * 11 / 10) > MaximumPayloadSize)
	{
		UE_LOGF(LogAnalytics, VeryVerbose, "AddToCache for event (%ls) may overflow MaximumPayloadSize (%d). Payload is currently (%d) bytes, and event will use an estimated (%d) bytes. Queuing up existing payload for flush before adding this event.", *EventName, MaximumPayloadSize, CachedEventUTF8Stream.Num(), EventSizeEstimate);
		QueueFlush();
	}

	// reserve enough space for the new data (an estimate, but should work fine if not a lot of UNICODE and Json escaping)
	const int32 OldBufferSize = CachedEventUTF8Stream.Num();
	CachedEventUTF8Stream.Reserve(CachedEventUTF8Stream.Num() + EventSizeEstimate + 10);

	// We will use this to esacpe the Json of our strings to avoid allocations.
	EventCacheStatic::FJsonStringBuilder EscapedJsonBuffer;

	// strip the payload tail off
	CachedEventUTF8Stream.SetNum(CachedEventUTF8Stream.Num() - EventCacheStatic::PayloadTrailerLength, EAllowShrinking::No);
	if (CachedEventEntries.Num() > 0)
	{
		// If we already have an event in there, start with a comma.
		CachedEventUTF8Stream.Add(static_cast<uint8>(','));
	}
	// Add {"EventName":
	EventCacheStatic::AppendString(CachedEventUTF8Stream, "{\"EventName\":", 13);
	// Add "<EVENTNAME>"
	EventCacheStatic::AppendJsonString(CachedEventUTF8Stream, EscapedJsonBuffer, EventName, false);
	// Add ,"DateOffset":"
	EventCacheStatic::AppendString(CachedEventUTF8Stream, ",\"DateOffset\":\"", 15);
	// record the location of this offset
	const int32 DateOffsetByteOffset = CachedEventUTF8Stream.Num();
	// add reserved space for the offset: +00:00:00.000"
	EventCacheStatic::AppendString(CachedEventUTF8Stream, "+00:00:00.000\"", 14);
	// append default attributes
	CachedEventUTF8Stream.Append(CachedDefaultAttributeUTF8Stream);
	// for each attribute, add ,"<NAME>":<VALUE>
	for (const FAnalyticsEventAttribute& Attr : Attributes)
	{
		EventCacheStatic::AppendEventAttribute(CachedEventUTF8Stream, EscapedJsonBuffer, Attr);
	}
	// Add }
	CachedEventUTF8Stream.Add(static_cast<uint8>('}'));
	// put the payload trailer back on
	EventCacheStatic::AppendString(CachedEventUTF8Stream, EventCacheStatic::PayloadTrailer, EventCacheStatic::PayloadTrailerLength);
	const int32 NewBufferSize = CachedEventUTF8Stream.Num();

	// Add the EventEntry
	CachedEventEntries.Add(FAnalyticsEventEntry(MoveTemp(EventName), DateOffsetByteOffset, NewBufferSize - OldBufferSize));
}

void FAnalyticsProviderETEventCache::AddToCache(FString EventName)
{
	AddToCache(MoveTemp(EventName), TArray<FAnalyticsEventAttribute>());
}

void FAnalyticsProviderETEventCache::SetDefaultAttributes(TArray<FAnalyticsEventAttribute>&& DefaultAttributes)
{
	UE::TScopeLock ScopedLock(CachedEventsCS);

	// store the array so we can return if if the user asks again.
	CachedDefaultAttributes = MoveTemp(DefaultAttributes);

	// presize the UTF8 stream that will store the pre-serialized default attribute buffer
	const int32 EstimatedAttributesSize = EventCacheStatic::ComputeAttributeSize(CachedDefaultAttributes) + 10;
	CachedDefaultAttributeUTF8Stream.Reset(EstimatedAttributesSize);
	if (CachedDefaultAttributes.Num() > 0)
	{
		EventCacheStatic::FJsonStringBuilder EscapedJsonBuffer;
		for (const FAnalyticsEventAttribute& Attr : CachedDefaultAttributes)
		{
			EventCacheStatic::AppendEventAttribute(CachedDefaultAttributeUTF8Stream, EscapedJsonBuffer, Attr);
		}
	}
}

TArray<FAnalyticsEventAttribute> FAnalyticsProviderETEventCache::GetDefaultAttributes() const
{
	UE::TScopeLock ScopedLock(CachedEventsCS);
	return CachedDefaultAttributes;
}

int32 FAnalyticsProviderETEventCache::GetDefaultAttributeCount() const
{
	UE::TScopeLock ScopedLock(CachedEventsCS);
	return CachedDefaultAttributes.Num();
}

FAnalyticsEventAttribute FAnalyticsProviderETEventCache::GetDefaultAttribute(int32 AttributeIndex) const
{
	UE::TScopeLock ScopedLock(CachedEventsCS);
	return CachedDefaultAttributes[AttributeIndex];
}

FString FAnalyticsProviderETEventCache::FlushCache(SIZE_T* OutEventCount)
{
	UE::TScopeLock ScopedLock(CachedEventsCS);
	if (OutEventCount)
	{
		*OutEventCount = CachedEventEntries.Num();
	}
	TArray<uint8> Payload = FlushCacheUTF8();
	Payload.Add(TEXT('\0'));
	return UTF8_TO_TCHAR(Payload.GetData());
}

TArray<uint8> FAnalyticsProviderETEventCache::FlushCacheUTF8()
{
	UE::TScopeLock ScopedLock(CachedEventsCS);

	// if there's nothing queued up, flush what we have.
	if (FlushQueue.Num() == 0 && CachedEventEntries.Num() > 0)
	{
		QueueFlush();
	}

	if (FlushQueue.Num() > 0)
	{
		// pull out the first element without copying the array or shrinking the queue size
		TArray<uint8> Payload = MoveTemp(FlushQueue[0]);
		FlushQueue.RemoveAt(0, EAllowShrinking::No);
		return Payload;
	}

	return TArray<uint8>();
}

TArray<uint8> FAnalyticsProviderETEventCache::CreateImmediatePayload(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes) const
{
	TSharedRef<FAnalyticsProviderETEventCache> Clone = CloneForImmediateMode();
	Clone->AddToCache(EventName, Attributes);
	return Clone->FlushCacheUTF8();
	// the Clone will be released here
}

// !!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!
// !!!! This method tries extremely hard to avoid any dynamic allocations
// !!!! to optimize the flush time. Please don't add new allocations to this function 
// !!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!
void FAnalyticsProviderETEventCache::QueueFlush()
{
	const double StartTime = FPlatformTime::Seconds();
	UE::TScopeLock ScopedLock(CachedEventsCS);

	// early exit if nothing to flush.
	if (CachedEventEntries.Num() == 0)
	{
		return;
	}

	const FDateTime CurrentTime = FDateTime::UtcNow();

	// The only thing we have to do is go through each event and fix up the DateOffset
	for (const FAnalyticsEventEntry& Entry : CachedEventEntries)
	{
		FTimespan DateOffset = CurrentTime - Entry.TimeStamp;
		// clamp thee timespan > 0 and less than 1 day.
		if (EventCacheStatic::bUseZeroDateOffset || DateOffset.GetTicks() < 0)
		{
			DateOffset = FTimespan(0);
		}
		else if (DateOffset.GetTotalDays() > 1.0)
		{
			DateOffset = FTimespan(23, 59, 59);
		}
		// implemnt our our ToString() directly into ANSICHARs, overwriting the placeholder Timespan we put there earlier.
		// Easiest to sprintf to a temp buffer that will null-terminate, then copy that into place.
		ANSICHAR DateOffsetBuf[14];
		FCStringAnsi::Snprintf(DateOffsetBuf, 14, "+%02i:%02i:%02i.%03i",
			FMath::Abs(DateOffset.GetHours()),
			FMath::Abs(DateOffset.GetMinutes()),
			FMath::Abs(DateOffset.GetSeconds()),
			FMath::Abs(DateOffset.GetFractionMilli()));
		FPlatformMemory::Memcpy(&CachedEventUTF8Stream[Entry.DateOffsetByteOffset], DateOffsetBuf, UE_ARRAY_COUNT(DateOffsetBuf) - 1); // don't copy the null
	}

	// see if it took too long or we have a really large payload. If so, log out the events.
	const double EndTime = FPlatformTime::Seconds();
	const bool bPlayloadTooLarge = CachedEventUTF8Stream.Num() > (int32)((float)MaximumPayloadSize * EventCacheStatic::PayloadPercentageOfMaxForWarning);
	const bool bTookTooLongToFlush = (EndTime - StartTime) > EventCacheStatic::PayloadFlushTimeSecForWarning;
	if (bPlayloadTooLarge)
	{
		
		UE_LOGF(LogAnalytics, Warning, "EventCache payload exceeded the maximum allowed size (%.3f KB > %.3f KB), containing %d events. Listing events in the payload for investigation:",
			(float)CachedEventUTF8Stream.Num() / 1024.f,
			((float)MaximumPayloadSize * EventCacheStatic::PayloadPercentageOfMaxForWarning) / 1024.f,
			CachedEventEntries.Num());
		for (const FAnalyticsEventEntry& Entry : CachedEventEntries)
		{
			UE_LOGF(LogAnalytics, Warning, "    %ls,%d", *Entry.EventName, Entry.EventSizeChars);
		}
	}
	// If the event took too long to flush, this may cause it to come up during profiling sessions. But generally, the problem is not with the telemetry code,
	// the problem is with Events that are trying to send too much data. List the events here to make it a bit easier to track down the responsible party for the slow telemetry.
	// Don't log at warning level because a lot automated tools don't care if telemetry flushes slowly, and it may happen in practice, and those tools will also error and
	// break the build if they detect warnings or errors.
	else if (bTookTooLongToFlush)
	{
		UE_LOGF(LogAnalytics, Display, "EventCache took too long to flush (%.3f ms > %.3f ms). Payload size: %.3f KB, %d events. Listing events in the payload for investigation:",
			(EndTime - StartTime) * 1000, EventCacheStatic::PayloadFlushTimeSecForWarning * 1000,
			(float)CachedEventUTF8Stream.Num() / 1024.f, CachedEventEntries.Num());
		for (const FAnalyticsEventEntry& Entry : CachedEventEntries)
		{
			UE_LOGF(LogAnalytics, Display, "    %ls,%d", *Entry.EventName, Entry.EventSizeChars);
		}
	}

	// clear out the old data
	CachedEventEntries.Reset();
	FlushQueue.Add(MoveTemp(CachedEventUTF8Stream));
	// reset our payload with the empty payload template. This will incure an allocation, which is the only allocation this function makes.
	EventCacheStatic::InitializePayloadBuffer(CachedEventUTF8Stream, PreallocatedPayloadSize);
}

TSharedRef<FAnalyticsProviderETEventCache> FAnalyticsProviderETEventCache::CloneForImmediateMode() const
{
	// need to make this because FAnalyticsProviderETEventCache is not allowed to be copyed because
	// FTransactionallySafeCriticalSection CachedEventsCS depends on FPlatformSharedMutex which has deleted copy and assignment operators.
	TSharedRef<FAnalyticsProviderETEventCache> Copy = MakeShared<FAnalyticsProviderETEventCache>(MaximumPayloadSize, PreallocatedPayloadSize);
	Copy->CachedDefaultAttributeUTF8Stream = CachedDefaultAttributeUTF8Stream;
	return Copy;
}

bool FAnalyticsProviderETEventCache::CanFlush() const
{
	UE::TScopeLock ScopedLock(CachedEventsCS);
	return CachedEventEntries.Num() > 0 || FlushQueue.Num() > 0;
}

bool FAnalyticsProviderETEventCache::HasFlushesQueued() const
{
	return FlushQueue.Num() > 0;
}

int FAnalyticsProviderETEventCache::GetNumCachedEvents() const
{
	UE::TScopeLock ScopedLock(CachedEventsCS);
	return CachedEventEntries.Num();
}

void FAnalyticsProviderETEventCache::SetPreallocatedPayloadSize(int32 InPreallocatedPayloadSize)
{
	PreallocatedPayloadSize = InPreallocatedPayloadSize;
	if (PreallocatedPayloadSize < 0)
	{
		PreallocatedPayloadSize = MaximumPayloadSize;
	}
	// if we are asking for a smaller buffer try to accommodate immediately.
	if (PreallocatedPayloadSize < (int32)CachedEventUTF8Stream.GetAllocatedSize())
	{
		UE::TScopeLock ScopedLock(CachedEventsCS);
		TArray<uint8> NewPayload;
		NewPayload.Reserve(PreallocatedPayloadSize);
		NewPayload = CachedEventUTF8Stream;
		CachedEventUTF8Stream = NewPayload;
	}
}

int32 FAnalyticsProviderETEventCache::GetSetPreallocatedPayloadSize() const
{
	return PreallocatedPayloadSize;
}

// Automation tests
#if WITH_DEV_AUTOMATION_TESTS
#include "Misc/AutomationTest.h"
#include <limits>

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAnalyticsProviderETEventCacheTest, "System.Analytics.AnalyticsETEventCache", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool FAnalyticsProviderETEventCacheTest::RunTest(const FString& Parameters)
{
	// Zero out the DateOffset so we can test against constant strings.
	TGuardValue<bool> GuardTestSetting(EventCacheStatic::bUseZeroDateOffset, true);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	{
		FString TheTestName = "BasicStrings";
		FAnalyticsProviderETEventCache cache;
		cache.AddToCache(TheTestName, MakeAnalyticsEventAttributeArray(
			TEXT("ConstantStringAttribute"), TEXT("ConstantStringValue"),
			TEXT("FStringStringAttribute"), FString("FStringValue")
		));

		FString ExpectedResult = "{\"Events\":[{\"EventName\":\"BasicStrings\",\"DateOffset\":\"+00:00:00.000\",\"ConstantStringAttribute\":\"ConstantStringValue\",\"FStringStringAttribute\":\"FStringValue\"}]}";
		TestEqual(TheTestName, cache.FlushCache(), ExpectedResult);
	}

	{
		FString TheTestName = "UnicodeEvent";
		FAnalyticsProviderETEventCache cache;
		FString Unicodestring(TEXT("\u0639\u0627\u0631\u0643\u0646\u064A\u0020\u0628\u0627\u0644\u0628\u0646\u0627\u0621\u0020\u6226\u3044"));
		cache.AddToCache(TheTestName, MakeAnalyticsEventAttributeArray(TEXT("UnicodeAttr"), Unicodestring));

		FString ExpectedResult = TEXT("{\"Events\":[{\"EventName\":\"UnicodeEvent\",\"DateOffset\":\"+00:00:00.000\",\"UnicodeAttr\":\"\u0639\u0627\u0631\u0643\u0646\u064A\u0020\u0628\u0627\u0644\u0628\u0646\u0627\u0621\u0020\u6226\u3044\"}]}");
		TestEqual(TheTestName, cache.FlushCache(), ExpectedResult);
	}

	{
		FString TheTestName = "NumericalEvent";
		FAnalyticsProviderETEventCache cache;
		cache.AddToCache(TheTestName, MakeAnalyticsEventAttributeArray(
			TEXT("IntAttr"), std::numeric_limits<int32>::min(),
			TEXT("LongAttr"), std::numeric_limits<int64>::min(),
			TEXT("UIntAttr"), std::numeric_limits<uint32>::max(),
			TEXT("ULongAttr"), std::numeric_limits<uint64>::max(),
			TEXT("FloatAttr"), std::numeric_limits<float>::max(),
			TEXT("DoubleAttr"), std::numeric_limits<double>::max(),
			TEXT("IntAttr2"), 0,
			TEXT("FloatAttr2"), 0.0f,
			TEXT("DoubleAttr2"), 0.0,
			TEXT("BoolTrueAttr"), true,
			TEXT("BoolFalseAttr"), false,
			// these need to end up null because json can't represent them.
			TEXT("INFAttr"), std::numeric_limits<double>::infinity(),
			TEXT("NANAttr"), std::numeric_limits<double>::quiet_NaN()
		));

		FString ExpectedResult = "{\"Events\":[{\"EventName\":\"NumericalEvent\",\"DateOffset\":\"+00:00:00.000\",\"IntAttr\":-2147483648,\"LongAttr\":-9223372036854775808,\"UIntAttr\":4294967295,\"ULongAttr\":18446744073709551615,\"FloatAttr\":3.402823466e+38,\"DoubleAttr\":1.797693135e+308,\"IntAttr2\":0,\"FloatAttr2\":0.0,\"DoubleAttr2\":0.0,\"BoolTrueAttr\":true,\"BoolFalseAttr\":false,\"INFAttr\":null,\"NANAttr\":null}]}";
		TestEqual(TheTestName, cache.FlushCache(), ExpectedResult);
	}

	{
		FString TheTestName = "JsonEvent";
		FAnalyticsProviderETEventCache cache;
		cache.AddToCache(TheTestName, MakeAnalyticsEventAttributeArray
		(
			TEXT("NullAttr"), FJsonNull(),
			TEXT("FragmentAttr"), FJsonFragment(TEXT("{\"Key\":\"Value\",\"Key2\":\"Value2\"}"))
		));

		FString ExpectedResult = "{\"Events\":[{\"EventName\":\"JsonEvent\",\"DateOffset\":\"+00:00:00.000\",\"NullAttr\":null,\"FragmentAttr\":{\"Key\":\"Value\",\"Key2\":\"Value2\"}}]}";
		TestEqual(TheTestName, cache.FlushCache(), ExpectedResult);

		return true;
	}

	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

#endif
