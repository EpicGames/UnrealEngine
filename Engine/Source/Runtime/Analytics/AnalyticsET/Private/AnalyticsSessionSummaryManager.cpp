// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnalyticsSessionSummaryManager.h"
#include "AnalyticsPropertyStore.h"
#include "IAnalyticsSessionSummarySender.h"
#include "GenericPlatform/GenericPlatformProcess.h"
#include "HAL/CriticalSection.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformTime.h"
#include "Misc/App.h"
#include "Misc/Optional.h"
#include "Misc/Paths.h"
#include "Internationalization/Regex.h"

DEFINE_LOG_CATEGORY_STATIC(LogAnalyticsSessionSummary, Verbose, All);

namespace AnalyticsManagerProperties
{
	// Those values are added by the manager for internal usage. They are stripped before sending the summary.
	static const FString InternalPropertyPrefix                        = "Internal.";
	static const TAnalyticsProperty<FString> InternalSessionUserId     = TEXT("Internal.UserId");
	static const TAnalyticsProperty<FString> InternalSessionAppId      = TEXT("Internal.AppId");
	static const TAnalyticsProperty<FString> InternalSessionAppVersion = TEXT("Internal.AppVersion");
	static const TAnalyticsProperty<FString> InternalSessionId         = TEXT("Internal.SessionId");
	static const TAnalyticsProperty<bool>    InternalWasProcessed      = TEXT("Internal.WasProcessed");

	// Those values are implicitely added by the manager into the report because the analytics backend expecpts them.
	static const TAnalyticsProperty<FString> SessionId       = TEXT("SessionId");
	static const TAnalyticsProperty<FString> ShutdownType    = TEXT("ShutdownType");
	static const TAnalyticsProperty<bool>    DelayedSend     = TEXT("DelayedSend");
	static const TAnalyticsProperty<FString> SentFrom        = TEXT("SentFrom");
	static const TAnalyticsProperty<FString> MissingDataFrom = TEXT("MissingDataFrom");

	// List of reserved property key.
	static TSet<FString> ReservedKeys = {
			AnalyticsManagerProperties::ShutdownType.Key,
			AnalyticsManagerProperties::DelayedSend.Key,
			AnalyticsManagerProperties::SentFrom.Key,
			AnalyticsManagerProperties::MissingDataFrom.Key
		};

	bool IsReserved(const FString& Key)
	{
		return ReservedKeys.Contains(Key);
	}
} // namespace AnalyticsManagerProperties


namespace AnalyticsSessionUtils
{
	static const FTimespan OrphanSessionCheckPeriod = FTimespan::FromMinutes(5);
	static const FTimespan ProcessGroupDeathGracePeriod = FTimespan::FromHours(1);

	static const FString OrphanSessionsOwnerCriticalSectionName = "UE_AnalyticsSessionSummaryManager_OrphanOwner";
	static const FString OrphanSessionsOwnerFilename            = "8E1D46DBC38F4A789939D781E1B91520"; // A randomly generated GUID.

	FString GetDefaultSavedDir()
	{
		// NOTE: This needs to be a folder common to Epic applications (Editor, CRC, etc.), ideally agnostics from the engine versions to enable picking orphans from
		//       older versions AND being constant between build configurations (like Editor Developement and CRC Shipping). FPaths functions are sensitive to the
		//       build configuration and will put files in different places.
		return FPaths::Combine(FPlatformProcess::UserSettingsDir(), *FApp::GetEpicProductIdentifier(), TEXT("Common"), TEXT("Analytics"));
	}

	FRegexPattern GetAnalyticsPropertyFilenamePattern()
	{
		// Filename format is like "ProcessGroupGUID_PID_PID_StoreCount_ProcessName", for example 3E1D7ADBC38F4A789939D781E1B91520_12345_232513_0_Editor"
		return FRegexPattern(TEXT(R"((^[0-9A-Za-z]+)_([0-9]+)_([0-9]+)(_[0-9]+)?_(.*))")); // Need help with regex? Try https://regex101.com/
	}

	/** Try to acquire the rigth to process and send sessions for orphan groups. */
	bool TryAcquiringOrphanGroupsOwnership(const FString& SessionRootPath, const FString& PrincipalProcessName, uint32 CurrentProcessId)
	{
		// NOTE: The usage of FPlatformMisc::SetStoredValue()/GetStoredValue() was considered, but those APIs are not thread safe on Mac/Linux
		//       and concurrent processes can easily stomp the changes made by another one. The implementation below prevent races between threads and processes.
		FAnalyticsPropertyStore OwnerStore;
		FString OrphanOwnerFilePathname = SessionRootPath / AnalyticsSessionUtils::OrphanSessionsOwnerFilename;

		bool bAcquiredOwnership = false;

		// Try to lock. Do not wait if the lock is already taken. That's not a big deal if we delay processing left-over sessions.
		FSystemWideCriticalSection SysWideLock(AnalyticsSessionUtils::OrphanSessionsOwnerCriticalSectionName, FTimespan::Zero());
		if (SysWideLock.IsValid())
		{
			if (IFileManager::Get().FileExists(*OrphanOwnerFilePathname))
			{
				if (OwnerStore.Load(OrphanOwnerFilePathname))
				{
					IAnalyticsPropertyStore::EStatusCode Status = OwnerStore.Set(PrincipalProcessName, CurrentProcessId,
						[](const uint32* Actual, const int32& ProposedValue) { return Actual == nullptr || *Actual == 0 || !FPlatformProcess::IsApplicationRunning(*Actual); });

					bAcquiredOwnership = (Status == IAnalyticsPropertyStore::EStatusCode::Success);
				}
				else // If the file exists but fails to load while this process as the system wide lock, that's likely because its content is invalid/corrupted.
				{
					IFileManager::Get().Delete(*OrphanOwnerFilePathname);
				}
			}
			else if (OwnerStore.Create(OrphanOwnerFilePathname, /*Capacity*/1 * 1024))
			{
				bAcquiredOwnership = OwnerStore.Set(PrincipalProcessName, CurrentProcessId) == IAnalyticsPropertyStore::EStatusCode::Success;
			}

			if (bAcquiredOwnership)
			{
				OwnerStore.Flush(); // Ensure to persist the storage to disk.
			}
		}

		return bAcquiredOwnership;
	}

	/** Release the ownership to send sessions for orphan groups. */
	void ReleaseOrphanGroupsOwnership(const FString& SessionRootPath, const FString& PrincipalProcessName, uint32 CurrentProcessId)
	{
		FString OrphanOwnerFilePathname = SessionRootPath / AnalyticsSessionUtils::OrphanSessionsOwnerFilename;
		if (IFileManager::Get().FileExists(*OrphanOwnerFilePathname))
		{
			// Try to lock. Do not wait if the lock is already taken.
			FSystemWideCriticalSection SysWideLock(AnalyticsSessionUtils::OrphanSessionsOwnerCriticalSectionName, FTimespan::Zero());
			if (SysWideLock.IsValid())
			{
				// Open the file.
				FAnalyticsPropertyStore OwnerStore;
				if (OwnerStore.Load(OrphanOwnerFilePathname))
				{
					// Reset to zero for this principal process.
					IAnalyticsPropertyStore::EStatusCode Status = OwnerStore.Set(PrincipalProcessName, static_cast<uint32>(0),
						[CurrentProcessId](const uint32* Actual, const uint32& Proposed) { return Actual == nullptr || *Actual == CurrentProcessId; });

					if (Status == IAnalyticsPropertyStore::EStatusCode::Success)
					{
						OwnerStore.Flush();
					}
				}
			}
		}
	}

} // namespace AnalyticsSessionUtils


const TCHAR* LexToString(EAnalyticsSessionShutdownType ShutdownTypeCode)
{
	switch (ShutdownTypeCode)
	{
	case EAnalyticsSessionShutdownType::Shutdown:
		return TEXT("Shutdown");

	case EAnalyticsSessionShutdownType::Terminated:
		return TEXT("Terminated");

	case EAnalyticsSessionShutdownType::Debugged:
		return TEXT("Debugger");

	case EAnalyticsSessionShutdownType::Crashed:
		return TEXT("Crashed");

	case EAnalyticsSessionShutdownType::Abnormal:
		return TEXT("AbnormalShutdown");

	case EAnalyticsSessionShutdownType::Hang:
		return TEXT("Hang");

	case EAnalyticsSessionShutdownType::Unknown:
	default:
		return TEXT("Unknown");
	}
}


const TAnalyticsProperty<int32> FAnalyticsSessionSummaryManager::ShutdownTypeCodeProperty = TEXT("ShutdownTypeCode");
const TAnalyticsProperty<bool>  FAnalyticsSessionSummaryManager::IsUserLoggingOutProperty = TEXT("IsUserLoggingOut");

FAnalyticsSessionSummaryManager::FAnalyticsSessionSummaryManager(const FString& InProcessName, const FString& InProcessGroupId,const FString& InUserId, const FString& InAppId, const FString& InAppVersion, const FString& InSessionId, const FString& InSavedDir)
	: FAnalyticsSessionSummaryManager(InProcessName, InProcessGroupId, FPlatformProcess::GetCurrentProcessId(), FPlatformProcess::GetCurrentProcessId(), InUserId, InAppId, InAppVersion, InSessionId, InSavedDir)
{
}

FAnalyticsSessionSummaryManager::FAnalyticsSessionSummaryManager(const FString& InProcessName, const FString& InProcessGroupId, uint32 InPrincipalProcessId, const FString& InSavedDir)
	: FAnalyticsSessionSummaryManager(InProcessName, InProcessGroupId, FPlatformProcess::GetCurrentProcessId(), InPrincipalProcessId, FString(), FString(), FString(), FString(), InSavedDir)
{
}

FAnalyticsSessionSummaryManager::FAnalyticsSessionSummaryManager(const FString& InProcessName, const FString& InProcessGroupId, uint32 InCurrentProcessId, uint32 InPrincipalProcessId, const FString& InUserId, const FString& InAppId, const FString& InAppVersion, const FString& InSessionId, const FString& InSavedDir)
	: ProcessName(InProcessName)
	, ProcessGroupId(InProcessGroupId)
	, UserId(InUserId)
	, AppId(InAppId)
	, AppVersion(InAppVersion)
	, SessionId(InSessionId)
	, SessionRootPath(InSavedDir)
	, CurrentProcessId(InCurrentProcessId)
	, PrincipalProcessId(InPrincipalProcessId)
	, StoreCounter(0)
	, NextOrphanSessionCheckTimeSecs(FPlatformTime::Seconds())
	, bOrphanGroupOwner(false)
	, bIsPrincipal(InCurrentProcessId == InPrincipalProcessId)
{
	if (SessionRootPath.IsEmpty() || !IFileManager::Get().DirectoryExists(*SessionRootPath))
	{
		SessionRootPath = AnalyticsSessionUtils::GetDefaultSavedDir(); // Fallback on the default.
	}

	// Creates a sub-folder to store the session files.
	IFileManager::Get().MakeDirectory(*SessionRootPath, /*Tree*/true);

	// Let principal process deals with its orphans. The principal process is expected to knows its application settings while a subsidiary process may not have any context.
	if (IsPrincipalProcess())
	{
		bOrphanGroupOwner = AnalyticsSessionUtils::TryAcquiringOrphanGroupsOwnership(SessionRootPath, ProcessName, CurrentProcessId);
	}
}

FAnalyticsSessionSummaryManager::~FAnalyticsSessionSummaryManager()
{
	if (bOrphanGroupOwner)
	{
		AnalyticsSessionUtils::ReleaseOrphanGroupsOwnership(SessionRootPath, ProcessName, CurrentProcessId);
	}
}

TSharedPtr<IAnalyticsPropertyStore> FAnalyticsSessionSummaryManager::MakeStore(uint32 InitialCapacity)
{
	// NOTE: The ProcessGroupId (a GUID) + PrincipalProcessId is expected to provide a unique key that should not collide with existing files.
	FString PropertyStorePathname = SessionRootPath / FString::Printf(TEXT("%s_%d_%d_%d_%s"), *ProcessGroupId, PrincipalProcessId, CurrentProcessId, StoreCounter, *ProcessName);

	TSharedPtr<FAnalyticsPropertyStore> PropertyStore = MakeShared<FAnalyticsPropertyStore>();
	if (PropertyStore && PropertyStore->Create(PropertyStorePathname, InitialCapacity))
	{
		// If this is the principal process for which the session summary is created.
		if (IsPrincipalProcess() && StoreCounter == 0)
		{
			// Add some internal key required to enable a subsidiary process to sent the summary on the behalf of this process.
			AnalyticsManagerProperties::InternalSessionUserId.Set(PropertyStore.Get(), UserId);
			AnalyticsManagerProperties::InternalSessionAppId.Set(PropertyStore.Get(), AppId);
			AnalyticsManagerProperties::InternalSessionAppVersion.Set(PropertyStore.Get(), AppVersion);
			AnalyticsManagerProperties::InternalSessionId.Set(PropertyStore.Get(), SessionId);
		}

		++StoreCounter;
	}
	else
	{
		PropertyStore.Reset(); // Ensure to return a null pointer if the store failed to create (not enough disk space for example)
	}

	WeakPropertyStores.Add(PropertyStore);
	PruneTrackedPropertyStores();

	return StaticCastSharedPtr<IAnalyticsPropertyStore>(PropertyStore);
}

void FAnalyticsSessionSummaryManager::SetSender(TSharedPtr<IAnalyticsSessionSummarySender> Sender)
{
	SummarySender = Sender;
}

void FAnalyticsSessionSummaryManager::SetUserId(const FString& InUserId)
{
	UserId = InUserId;

	PruneTrackedPropertyStores();

	for (TWeakPtr<IAnalyticsPropertyStore>& WeakPropertyStore : WeakPropertyStores)
	{
		if (TSharedPtr<IAnalyticsPropertyStore> PropertyStore = WeakPropertyStore.Pin())
		{
			AnalyticsManagerProperties::InternalSessionUserId.Set(PropertyStore.Get(), UserId);
			PropertyStore->Flush();
		}
	}
}

void FAnalyticsSessionSummaryManager::Shutdown(bool bDiscard)
{
	if (SummarySender)
	{
		// Check if this process has something to summarize and send for this instance process group.
		TMap<FString, FProcessGroup> PropertyFiles = GetSessionFiles();
		if (const FProcessGroup* ProcessGroup = PropertyFiles.Find(ProcessGroupId))
		{
			if (bDiscard)
			{
				CleanupFiles(ProcessGroup->PropertyFiles, /*bOnSuccess*/false);
			}
			else
			{
				ProcessSummary(ProcessGroupId, *ProcessGroup);
			}
		}
	}
	else if (bDiscard)
	{
		TMap<FString, FProcessGroup> PropertyFiles = GetSessionFiles();
		if (const FProcessGroup* ProcessGroup = PropertyFiles.Find(ProcessGroupId))
		{
			CleanupFiles(ProcessGroup->PropertyFiles, /*bOnSuccess*/false);
		}
	}

	StoreCounter = 0;
}

void FAnalyticsSessionSummaryManager::Tick()
{
	if (!SummarySender || !IsOrphanGroupsOwner())
	{
		return; // Don't process orphans if the result cannot be sent.
	}

	if (FPlatformTime::Seconds() > NextOrphanSessionCheckTimeSecs)
	{
		NextOrphanSessionCheckTimeSecs = FPlatformTime::Seconds() + AnalyticsSessionUtils::OrphanSessionCheckPeriod.GetTotalSeconds();

		// Check if this process has something to summarize and send..
		TMap<FString, FProcessGroup> PropertyFiles = GetSessionFiles();
		for (const TPair<FString, FProcessGroup>& Pair : PropertyFiles)
		{
			if (Pair.Value.PrincipalProcessId == PrincipalProcessId)
			{
				continue; // Skip this process session summary. It should not be sent until the last process in the group died.
			}

			// Picked up an orphan session, decide what to do with it.
			ProcessSummary(/*ProcessGroupId*/Pair.Key, /*ProcessGroup*/Pair.Value);
		}
	}
}

bool FAnalyticsSessionSummaryManager::IsPrincipalProcess() const
{
	return bIsPrincipal;
}

void FAnalyticsSessionSummaryManager::PruneTrackedPropertyStores()
{
	for (int32 index = WeakPropertyStores.Num() - 1; index >= 0; --index)
	{
		if (!WeakPropertyStores[index].IsValid())
		{
			WeakPropertyStores.RemoveAtSwap(index);
		}
	}
}

TMap<FString, FAnalyticsSessionSummaryManager::FProcessGroup> FAnalyticsSessionSummaryManager::GetSessionFiles() const
{
	TMap<FString, FProcessGroup> ProcessFiles;

	// Visit the files in the session directory.
	FRegexPattern FilenamePattern = AnalyticsSessionUtils::GetAnalyticsPropertyFilenamePattern();
	IFileManager::Get().IterateDirectory(*SessionRootPath, [this, &ProcessFiles, &FilenamePattern](const TCHAR* Pathname, bool bIsDirectory) -> bool
	{
		if (!bIsDirectory)
		{
			// The property filename is "ProcessGroupGUID_PrincipalProcessId_WriterProcessId_StoreCount_WriterProcessName". The ProcessGroupGUID is expected to have EGuidFormats::Digits -> "00000A00C0000F000000000000000000"
			FRegexMatcher Matcher(FilenamePattern, FPaths::GetCleanFilename(Pathname));
			if (Matcher.FindNext())
			{
				FString FileProcessGroupId = Matcher.GetCaptureGroup(1);
				if (FileProcessGroupId != ProcessGroupId && !IsOrphanGroupsOwner())
				{
					return true; // This file is from another process group and this manager is not allowed to process it, skip to the next file.
				}

				FPropertyFileInfo PropertyFileInfo;
				PropertyFileInfo.ProcessId = static_cast<uint32>(FCString::Atoi64(*Matcher.GetCaptureGroup(3)));
				PropertyFileInfo.ProcessName = Matcher.GetCaptureGroup(5);
				PropertyFileInfo.Pathname = Pathname;

				uint32 FilePrincipalProcessId = static_cast<uint32>(FCString::Atoi64(*Matcher.GetCaptureGroup(2)));
				FProcessGroup& ProcessGroup = ProcessFiles.FindOrAdd(FileProcessGroupId);
				ProcessGroup.PrincipalProcessId = FilePrincipalProcessId;

				if (FilePrincipalProcessId == PropertyFileInfo.ProcessId) // That file was produced by a principal process.
				{
					ProcessGroup.PrincipalProcessName = PropertyFileInfo.ProcessName;
				}

				ProcessGroup.PropertyFiles.Add(MoveTemp(PropertyFileInfo));
			}
		}
		return true; // Continue to the next file.
	});

	// Can this manager process orphan groups?
	if (IsOrphanGroupsOwner())
	{
		check(IsPrincipalProcess());

		for (TMap<FString, FProcessGroup>::TIterator It = ProcessFiles.CreateIterator(); It; ++It)
		{
			// Discard groups that are not from this applications.
			if (!It.Value().PrincipalProcessName.IsEmpty() && It.Value().PrincipalProcessName != ProcessName)
			{
				It.RemoveCurrent();
			}
			// Discard any other groups that are still running.
			else if (It.Value().PrincipalProcessId != PrincipalProcessId)
			{
				for (const FPropertyFileInfo& PropertyFileInfo : It.Value().PropertyFiles)
				{
					if (FPlatformProcess::IsApplicationRunning(PropertyFileInfo.ProcessId)) // One of the process in that group is still running
					{
						It.RemoveCurrent();
						break;
					}
				}
			}
		}
	}

	return ProcessFiles;
}

void FAnalyticsSessionSummaryManager::ProcessSummary(const FString& InProcessGroupId, const FProcessGroup& InProcessGroup)
{
	// Check if this process is the last alive in the group, so that it can process the summary
	for (const FPropertyFileInfo& PropertyFileInfo : InProcessGroup.PropertyFiles)
	{
		if (PropertyFileInfo.ProcessId != CurrentProcessId)
		{
			if (FPlatformProcess::IsApplicationRunning(PropertyFileInfo.ProcessId))
			{
				return; // A process in the group is running, leave this group alone.
			}
		}
	}

	// Trying to process a summary for another group than this manager group?
	if (InProcessGroupId != ProcessGroupId)
	{
		checkf(IsOrphanGroupsOwner(), TEXT("Should not process summary of other process group if this wasn't allowed in first place."));

		bool bExpired = false;

		// The files discovered tell that processes in the group are dead, but it is unclear if another process in that group is about to create a files or not (because
		// of race conditions between processes).
		for (const FPropertyFileInfo& PropertyFileInfo : InProcessGroup.PropertyFiles)
		{
			// Time since the last file modification.
			double FileAgeSecs = IFileManager::Get().GetFileAgeSeconds(*PropertyFileInfo.Pathname);

			// Before declaring this group 'dead' leave a reasonable grace period to clear 99% of possible race conditions between processes.
			if (FileAgeSecs > 0 && FileAgeSecs <= AnalyticsSessionUtils::ProcessGroupDeathGracePeriod.GetTotalSeconds())
			{
				return; // Wait more before declaring this group dead and processing it.
			}

			// If the principal file is expired, expire the entire group.
			if (InProcessGroup.PrincipalProcessId == PropertyFileInfo.ProcessId && FileAgeSecs > GetSessionExpirationAge().GetTotalSeconds())
			{
				bExpired = true;
				break;
			}
		}

		// This group is expired or doesn't have the properties from the principal process (incomplete session).
		if (bExpired || InProcessGroup.PrincipalProcessName.IsEmpty())
		{
			// The session data is partial or expired.
			CleanupFiles(InProcessGroup.PropertyFiles, /*bOnSuccess*/false);
			return;
		}
	}

	// Aggregate the principal and subsidiary process summaries.
	TArray<FAnalyticsEventAttribute> InternalProperties;
	TArray<FAnalyticsEventAttribute> SummaryProperties;
	if (!AggregateSummaries(InProcessGroupId, InProcessGroup, SummaryProperties, InternalProperties))
	{
		CleanupFiles(InProcessGroup.PropertyFiles, /*bOnSuccess*/false);
		return;
	}

	// Was the group aggregated above the group of this principal process?
	if (IsPrincipalProcess() && InProcessGroupId == ProcessGroupId)
	{
		// Clean up before sending, so that we don't send twice. (In case of crash while sending, we will not know if the summary was sent or not).
		CleanupFiles(InProcessGroup.PropertyFiles, /*bOnSuccess*/true);
		SummarySender->SendSessionSummary(UserId, AppId, AppVersion, SessionId, SummaryProperties);
	}
	// Send the session summary on behalf of the principal process that created it.
	else
	{
		// The principal process saves those internal properties to let a subsidiary or another principal processes submit the summary on its behalf.
		const FAnalyticsEventAttribute* UserIdProp = InternalProperties.FindByPredicate([](const FAnalyticsEventAttribute& Candidate) { return Candidate.GetName() == AnalyticsManagerProperties::InternalSessionUserId.Key; });
		const FAnalyticsEventAttribute* AppIdProp = InternalProperties.FindByPredicate([](const FAnalyticsEventAttribute& Candidate) { return Candidate.GetName() == AnalyticsManagerProperties::InternalSessionAppId.Key; });
		const FAnalyticsEventAttribute* AppVersionProp = InternalProperties.FindByPredicate([](const FAnalyticsEventAttribute& Candidate) { return Candidate.GetName() == AnalyticsManagerProperties::InternalSessionAppVersion.Key; });
		const FAnalyticsEventAttribute* SessionIdProp = InternalProperties.FindByPredicate([](const FAnalyticsEventAttribute& Candidate) { return Candidate.GetName() == AnalyticsManagerProperties::InternalSessionId.Key; });

		if (UserIdProp && AppIdProp && AppVersionProp && SessionIdProp)
		{
			CleanupFiles(InProcessGroup.PropertyFiles, /*bOnSuccess*/true);
			SummarySender->SendSessionSummary(UserIdProp->GetValue(), AppIdProp->GetValue(), AppVersionProp->GetValue(), SessionIdProp->GetValue(), SummaryProperties);
		}
		else
		{
			CleanupFiles(InProcessGroup.PropertyFiles, /*bOnSuccess*/false);
		}
	}
}

bool FAnalyticsSessionSummaryManager::AggregateSummaries(const FString& InProcessGroupId, const FProcessGroup& ProcessGroup, TArray<FAnalyticsEventAttribute>& OutSummaryProperties, TArray<FAnalyticsEventAttribute>& OutInternalProperties)
{
	FAnalyticsPropertyStore PropertyStore;

	TSet<FString> SummaryKeys;

	auto RenameKey = [](const FString& KeyToRename, const FString& FromProcess, const TSet<FString>& ExistingKeys, const TCHAR* Reason)
	{
		FString RenamedKey;
		int32 Index = 1;
		do
		{
			RenamedKey = FString::Printf(TEXT("%s_%d"), *KeyToRename, Index++);
		} while (ExistingKeys.Contains(RenamedKey));

		UE_LOGF(LogAnalyticsSessionSummary, Warning, "Renamed key '%ls' from process '%ls' as '%ls'. Reason: %ls", *KeyToRename, *FromProcess, *RenamedKey, Reason);
		return RenamedKey;
	};

	TArray<FString> MissingDataFromProcesses;

	// Merge the properties collected by the principal and its subsidiary processes.
	for (const FPropertyFileInfo& PropertyFileInfo : ProcessGroup.PropertyFiles)
	{
		if (PropertyStore.Load(PropertyFileInfo.Pathname))
		{
			if (PropertyStore.Contains(AnalyticsManagerProperties::InternalWasProcessed.Key))
			{
				return false; // We don't know if the manager failed to delete before of after the session was sent. Ignore this group.
			}

			int32 ShutdownTypeCode;
			if (PropertyStore.Get(ShutdownTypeCodeProperty.Key, ShutdownTypeCode) == IAnalyticsPropertyStore::EStatusCode::Success)
			{
				// Don't add 'ShutdownType' more than once. If 'ShutdownTypeCode' key used to derive the 'ShutdownType' is duplicated, the code will emit a warning and rename the second instance.
				if (!SummaryKeys.Contains(AnalyticsManagerProperties::ShutdownType.Key))
				{
					// Convert 'ShutdownTypeCode' into its string representation and emit it as 'ShutdownType' as known by the analytics backend.
					OutSummaryProperties.Emplace(AnalyticsManagerProperties::ShutdownType.Key, LexToString(static_cast<EAnalyticsSessionShutdownType>(ShutdownTypeCode)));
					SummaryKeys.Add(AnalyticsManagerProperties::ShutdownType.Key);
				}
			}

			PropertyStore.VisitAll([&PropertyFileInfo, &OutSummaryProperties, &OutInternalProperties, &RenameKey, &SummaryKeys](FAnalyticsEventAttribute&& Attr)
			{
				if (Attr.GetName().StartsWith(AnalyticsManagerProperties::InternalPropertyPrefix))
				{
					SummaryKeys.Add(Attr.GetName());
					OutInternalProperties.Emplace(Attr);
				}
				else if (AnalyticsManagerProperties::IsReserved(Attr.GetName()))
				{
					FString RenamedKey = RenameKey(Attr.GetName(), PropertyFileInfo.ProcessName, SummaryKeys, TEXT("Key name is reserved."));
					SummaryKeys.Add(RenamedKey);
					OutSummaryProperties.Emplace(RenamedKey, Attr.GetValue());
				}
				else if (!SummaryKeys.Contains(Attr.GetName()))
				{
					SummaryKeys.Add(Attr.GetName());
					OutSummaryProperties.Emplace(Attr);
				}
				else // Deal with key collisions. Keys are supposed to be unique. Duplication should be easy to spot during developement.
				{
					FString RenamedKey = RenameKey(Attr.GetName(), PropertyFileInfo.ProcessName, SummaryKeys, TEXT("Key collision between processes of a group."));
					SummaryKeys.Add(Attr.GetName());
					OutSummaryProperties.Emplace(RenamedKey, Attr.GetValue());
				}
			});

			// Mark this file as processed to avoid sending the same session twice.
			AnalyticsManagerProperties::InternalWasProcessed.Set(&PropertyStore, true);
			PropertyStore.Flush();
		}
		else if (PropertyFileInfo.ProcessId == ProcessGroup.PrincipalProcessId)
		{
			// Without the principal process data, too much data is missing, abort.
			return false;
		}
		else
		{
			// Subsidiary process data will be missing. Not ideal, but this should only be complementary info for the principal process, flag it and continue.
			MissingDataFromProcesses.Add(PropertyFileInfo.ProcessName);
		}
	}

	// The manager saves this key in the store it creates.
	if (const FAnalyticsEventAttribute* InternalSessionId = OutInternalProperties.FindByPredicate([](const FAnalyticsEventAttribute& Candidate) { return Candidate.GetName() == AnalyticsManagerProperties::InternalSessionId.Key; }))
	{
		OutSummaryProperties.Emplace(AnalyticsManagerProperties::SessionId.Key, InternalSessionId->GetValue());
	}

	if (MissingDataFromProcesses.Num())
	{
		// Add the subsidiary process tag name(s) for which the data file couldn't be loaded (file was locked, corrupted, ...)
		OutSummaryProperties.Emplace(AnalyticsManagerProperties::MissingDataFrom.Key, FString::Join(MissingDataFromProcesses, TEXT(",")));
	}

	// The summary is sent delayed when it is processed from another group.
	OutSummaryProperties.Emplace(AnalyticsManagerProperties::DelayedSend.Key, InProcessGroupId != ProcessGroupId);

	// This process is about to send the report, record the process name in the summary.
	OutSummaryProperties.Emplace(AnalyticsManagerProperties::SentFrom.Key, ProcessName);

	return true;
}

bool FAnalyticsSessionSummaryManager::CleanupFiles(const TArray<FPropertyFileInfo>& PropertyFiles, bool bOnSuccess)
{
	// TODO as future work: Compute how many times we clean up on failure. Keep track of it and periodically send an analytic event about it.

	bool bAllDeleted = true;
	for (const FPropertyFileInfo& PropertyFileInfo : PropertyFiles)
	{
		bAllDeleted &= IFileManager::Get().Delete(*PropertyFileInfo.Pathname, /*MustExist*/false, /*EvenReadOnly*/true, /*Quiet*/true);
	}
	return bAllDeleted;
}

void FAnalyticsSessionSummaryManager::CleanupExpiredFiles(const FString& SavedDir)
{
	FString SessionRootPath = SavedDir.IsEmpty() ? AnalyticsSessionUtils::GetDefaultSavedDir() : SavedDir;

	// The manager stores its files under its own folder.
	if (!IFileManager::Get().DirectoryExists(*SessionRootPath))
	{
		return;
	}

	// Visit the files in the session directory.
	FRegexPattern FilenamePattern = AnalyticsSessionUtils::GetAnalyticsPropertyFilenamePattern();
	IFileManager::Get().IterateDirectory(*SessionRootPath, [&FilenamePattern](const TCHAR* Pathname, bool bIsDirectory) -> bool
	{
		if (bIsDirectory)
		{
			return true; // Continue.
		}

		FRegexMatcher Matcher(FilenamePattern, FPaths::GetCleanFilename(Pathname));
		if (Matcher.FindNext())
		{
			// Time since the last file modification.
			double FileAgeSecs = IFileManager::Get().GetFileAgeSeconds(Pathname);

			// Add up an extra day to the normal expiration delay so that we never clean up files that are on the edge.
			if (FileAgeSecs > GetSessionExpirationAge().GetTotalSeconds() + FTimespan::FromDays(1.0).GetTotalSeconds())
			{
				IFileManager::Get().Delete(Pathname);
			}
		}

		return true;
	});
}
