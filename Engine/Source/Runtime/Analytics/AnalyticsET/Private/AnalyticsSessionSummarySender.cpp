// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnalyticsSessionSummarySender.h"
#include "AnalyticsET.h"
#include "AnalyticsEventAttribute.h"
#include "IAnalyticsProviderET.h"
#include "Misc/EngineVersion.h"

DEFINE_LOG_CATEGORY_STATIC(LogAnalyticsSessionSummarySender, Verbose, All);


FAnalyticsSessionSummarySender::FAnalyticsSessionSummarySender(IAnalyticsProviderET& Provider, TFunction<bool(const FAnalyticsEventAttribute&)> ShouldEmitFilterFunc)
	: AnalyticsProvider(Provider)
	, ShouldEmitPropFunc(MoveTemp(ShouldEmitFilterFunc))
{
}

bool FAnalyticsSessionSummarySender::SendSessionSummary(const FString& UserId, const FString& AppId, const FString& AppVersion, const FString& SessionId, const TArray<FAnalyticsEventAttribute>& Properties)
{
	TArray<FAnalyticsEventAttribute> AnalyticsAttributes;

	// Track which app/user is sending the summary (summary can be sent by another process (CrashReportClient) or another instance in case of crash/abnormal terminaison.
	AnalyticsAttributes.Emplace(TEXT("SummarySenderAppId"), AnalyticsProvider.GetAppID());
	AnalyticsAttributes.Emplace(TEXT("SummarySenderAppVersion"), AnalyticsProvider.GetAppVersion());
	AnalyticsAttributes.Emplace(TEXT("SummarySenderEngineVersion"), FEngineVersion::Current().ToString(EVersionComponent::Changelist)); // Same as in EditorSessionSummaryWriter.cpp
	AnalyticsAttributes.Emplace(TEXT("SummarySenderUserId"), AnalyticsProvider.GetUserID());
	AnalyticsAttributes.Emplace(TEXT("SummarySenderSessionId"), AnalyticsProvider.GetSessionID()); // Not stripping the {} around the GUID like EditorSessionSummaryWriter does with SessionId.

	for (const FAnalyticsEventAttribute& Property : Properties)
	{
		// Function is not bound or returns true.
		if (!ShouldEmitPropFunc || ShouldEmitPropFunc(Property))
		{
			AnalyticsAttributes.Emplace(Property);
		}
	}

	// Sort the keys to makes it easier to find specific ones when inspecting the network payload (mainly for testing/debugging purpose).
	AnalyticsAttributes.Sort([](const FAnalyticsEventAttribute& Lhs, const FAnalyticsEventAttribute& Rhs){ return Lhs.GetName() < Rhs.GetName(); });

	// Sending the summary event of the current process analytic session?
	if (AnalyticsProvider.GetSessionID().Contains(SessionId)) // The string (GUID) returned by GetSessionID() is surrounded with braces like "{3FEA3232-...}" while Session.SessionId is not -> "3FEA3232-..."
	{
		AnalyticsProvider.RecordEvent(TEXT("SessionSummary"), AnalyticsAttributes);
	}
	else // The summary was created by another process/instance in a different session. (Ex: Editor sending a summary of a  previoulsy crashed instance or CrashReportClient sending it on behalf of the Editor)
	{
		// The provider sending a 'summary event' created by another instance/process must parametrize its post request 'as if' it was sent from the instance/session that created it (backend expectation).
		// Create a new provider to avoid interfering with the current session events. (ex. if another thread sends telemetry at the same time, don't accidently tag it with the wrong SessionID, AppID, etc.).
		TSharedPtr<IAnalyticsProviderET> TempSummaryProvider = FAnalyticsET::Get().CreateAnalyticsProvider(AnalyticsProvider.GetConfig());

		// Reconfigure the analytics provider to sent the summary event 'as if' it was sent by the process that created it. This is required by the analytics backend.
		FGuid SessionGuid;
		FGuid::Parse(*SessionId, SessionGuid);
		TempSummaryProvider->SetUserID(*UserId);
		TempSummaryProvider->SetAppID(*AppId);
		TempSummaryProvider->SetAppVersion(*AppVersion);
		TempSummaryProvider->SetSessionID(SessionGuid.ToString(EGuidFormats::DigitsWithHyphensInBraces)); // Ensure to put back the {} around the GUID.

		// Send the summary.
		TempSummaryProvider->RecordEvent(TEXT("SessionSummary"), AnalyticsAttributes);

		// The temporary provider is about to be deleted (going out of scope), ensure it sents its report.
		TempSummaryProvider->BlockUntilFlushed(2.0f);
	}

	// For debugging.
	//for (const FAnalyticsEventAttribute& Att : AnalyticsAttributes)
	//{
	//	UE_LOGF(LogAnalyticsSessionSummarySender, Display, "Attr %ls = %ls", *Att.GetName(), *Att.GetValue());
	//}

	UE_LOGF(LogAnalyticsSessionSummarySender, Log, "Sent summary session report for: AppId=%ls SessionId=%ls", *AppId, *SessionId);
	return true;
}
