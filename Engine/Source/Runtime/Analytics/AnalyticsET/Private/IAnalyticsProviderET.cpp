// Copyright Epic Games, Inc. All Rights Reserved.

#include "IAnalyticsProviderET.h"
#include "GenericPlatform/GenericPlatform.h"
#include "Misc/CommandLine.h"
#include "Misc/ScopeLock.h"
#include "Stats/Stats.h"
#include "Containers/Ticker.h"
#include "Misc/App.h"
#include "Misc/TimeGuard.h"

#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonWriter.h"
#include "AnalyticsProviderETEventCache.h"
#include "AnalyticsET.h"
#include "Analytics.h"
#include "Interfaces/IHttpResponse.h"
#include "Interfaces/IHttpRequest.h"
#include "HttpModule.h"
#include "PlatformHttp.h"
#include "Misc/EngineVersion.h"
#include "HttpRetrySystem.h"
#include "HAL/IConsoleManager.h"
#include "AutoRTFM.h"

#include "AnalyticsPerfTracker.h"

namespace AnalyticsProviderETCvars
{
	static bool PreventMultipleFlushesInOneFrame = true;
	FAutoConsoleVariableRef CvarPreventMultipleFlushesInOneFrame(
		TEXT("AnalyticsET.PreventMultipleFlushesInOneFrame"),
		PreventMultipleFlushesInOneFrame,
		TEXT("When true, prevents more than one AnalyticsProviderET instance from flushing in the same frame, allowing the flush and HTTP cost to be amortized.")
	);

	TAutoConsoleVariable<bool> CVarDefaultUserAgentCommentsEnabled(
		TEXT("AnalyticsET.UserAgentCommentsEnabled"),
		true,
		TEXT("Whether comments are supported in the analytics user agent string"),
		ECVF_SaveForNextBoot
	);
}

// Want to avoid putting the project name into the User-Agent, because for some apps (like the editor), the project name is private info.
// The analytics User-Agent uses the default User-Agent, but with project name removed.
class FAnalyticsUserAgentCache
{
public:
	FAnalyticsUserAgentCache()
		: CachedUserAgent()
		, CachedAgentVersion(0)
	{
	}

	FString GetUserAgent()
	{
		if (CachedUserAgent.IsEmpty() || CachedAgentVersion != FPlatformHttp::GetDefaultUserAgentVersion())
		{
			UpdateUserAgent();
		}

		return CachedUserAgent;
	}

private:
	void UpdateUserAgent()
	{
		static TSet<FString> AllowedProjectComments(GetAllowedProjectComments());
		static TSet<FString> AllowedPlatformComments(GetAllowedPlatformComments());

		FDefaultUserAgentBuilder Builder = FPlatformHttp::GetDefaultUserAgentBuilder();
		Builder.SetProjectName(TEXT("PROJECTNAME"));
		CachedUserAgent = Builder.BuildUserAgentString(&AllowedProjectComments, &AllowedPlatformComments);
		CachedAgentVersion = Builder.GetAgentVersion();
	}

	static TSet<FString> GetAllowedProjectComments()
	{
		TArray<FString> AllowedProjectComments;
		if (AnalyticsProviderETCvars::CVarDefaultUserAgentCommentsEnabled.GetValueOnAnyThread())
		{
			GConfig->GetArray(TEXT("Analytics"), TEXT("AllowedUserAgentProjectComments"), AllowedProjectComments, GEngineIni);
		}
		return TSet<FString>(MoveTemp(AllowedProjectComments));
	}

	static TSet<FString> GetAllowedPlatformComments()
	{
		TArray<FString> AllowedPlatformComments;
		if (AnalyticsProviderETCvars::CVarDefaultUserAgentCommentsEnabled.GetValueOnAnyThread())
		{
			GConfig->GetArray(TEXT("Analytics"), TEXT("AllowedUserAgentPlatformComments"), AllowedPlatformComments, GEngineIni);
		}
		return TSet<FString>(MoveTemp(AllowedPlatformComments));
	}

	static TSet<FString> ParseCommentSet(const FString& CommentBlob)
	{
		TArray<FString> Comments;
		CommentBlob.ParseIntoArray(Comments, TEXT(";"));
		return TSet<FString>(MoveTemp(Comments));
	}

	FString CachedUserAgent;
	uint32 CachedAgentVersion;
};

/**
 * Implementation of analytics for Epic Telemetry.
 * Supports caching events and flushing them periodically (currently hardcoded limits).
 * Also supports a set of default attributes that will be added to every event.
 * For efficiency, this set of attributes is added directly into the set of cached events
 * with a special flag to indicate its purpose. This allows the set of cached events to be used like
 * a set of commands to be executed on flush, and allows us to inject the default attributes
 * efficiently into many events without copying the array at all.
 * If Config.APIServerET is empty, this will act as a NULL provider by forcing ShouldRecordEvent() to return false all the time.
 */
class FAnalyticsProviderET :
	public IAnalyticsProviderET,
	public FTSTickerObjectBase,
	public TSharedFromThis<FAnalyticsProviderET>
{
public:
	FAnalyticsProviderET(const FAnalyticsET::Config& ConfigValues);

	// FTSTickerObjectBase

	bool Tick(float DeltaSeconds) override;

	// IAnalyticsProvider

	virtual bool StartSession(FString InSessionID, const TArray<FAnalyticsEventAttribute>& Attributes) override;
	virtual void EndSession() override;
	virtual void FlushEvents() override;

	virtual void SetAppID(FString&& AppID) override;
	virtual void SetAppVersion(FString&& AppVersion) override;
	virtual void SetUserID(const FString& InUserID) override;
	virtual FString GetUserID() const override;

	virtual FString GetSessionID() const override;
	virtual bool SetSessionID(const FString& InSessionID) override;

	virtual bool ShouldRecordEvent(const FString& EventName) const override;
	virtual void RecordEvent(FString&& EventName, const TArray<FAnalyticsEventAttribute>& Attributes) override;
	virtual void RecordEvent(FString&& EventName, const TArray<FAnalyticsEventAttribute>& Attributes, EAnalyticsRecordEventMode Mode) override;
	virtual void SetDefaultEventAttributes(TArray<FAnalyticsEventAttribute>&& Attributes) override;
	virtual TArray<FAnalyticsEventAttribute> GetDefaultEventAttributesSafe() const override;
	virtual int32 GetDefaultEventAttributeCount() const override;
	virtual FAnalyticsEventAttribute GetDefaultEventAttribute(int AttributeIndex) const override;
	virtual void SetEventCallback(const OnEventRecorded& Callback) override;

	virtual void SetUrlDomain(const FString& Domain, const TArray<FString>& AltDomains) override;
	virtual void SetUrlPath(const FString& Path) override;
	virtual void SetHeader(const FString& HeaderName, const FString& HeaderValue) override;
	virtual void BlockUntilFlushed(float InTimeoutSec) override;
	virtual void SetShouldRecordEventFunc(const ShouldRecordEventFunction& InShouldRecordEventFunc) override;
	virtual FOnPreAnalyticsEventProcessed& OnPreAnalyticsEventProcessed() override { return OnPreAnalyticsEventProcessedDelegate; }
	virtual FOnAnalyticsEventQueued& OnAnalyticsEventQueued() override { return OnAnalyticsEventQueuedDelegate; }

	virtual ~FAnalyticsProviderET();

	virtual const FAnalyticsET::Config& GetConfig() const override { return Config; }

private:
	void ExecuteRequest(TArray<uint8>& Payload, OUT int32& PayloadSize, OUT int32& EventCount, EAnalyticsRecordEventMode Mode);
	void SendImmediately(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes);
	void FlushEventsOnce();
	void FlushEventLegacy(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes);
	bool IsActingAsNullProvider() const
	{
		// if we don't have a primary APIKey then we are essentially acting as a NULL provider and will suppress all events.
		// Don't bother checking the retry domains because the primary domain being empty is enough to tell us we have nowhere to send as a primary destination.
		return Config.APIServerET.IsEmpty();
	}

	/** Create a request utilizing HttpRetry domains */
	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> CreateRequest(EAnalyticsRecordEventMode Mode = EAnalyticsRecordEventMode::Cached);

	bool bSessionInProgress;
	/** The current configuration (might be updated with respect to the one provided at construction). */
	FAnalyticsET::Config Config;
	/** the unique UserID as passed to ET. */
	FString UserID;
	/** The session ID */
	FString SessionID;
	/** Default flush interval, when one is not explicitly given. */
	const float DefaultFlushIntervalSec = 60.0f;
	/** interval which to ensure events are flushed to the server. An event should not sit in the cache longer than this. It may be flushed sooner, but not longer (unless there is a hitch) */
	float FlushIntervalSec;
	/** Allows events to not be cached when -AnalyticsDisableCaching is used. This should only be used for debugging as caching significantly reduces bandwidth overhead per event. */
	bool bShouldCacheEvents;
	/** Current timer to keep track of FlushIntervalSec flushes */
	double NextEventFlushTime;
	/** Track destructing for unbinding callbacks when firing events at shutdown */
	bool bInDestructor;

	FAnalyticsProviderETEventCache EventCache;

	TArray<OnEventRecorded> EventRecordedCallbacks;

	/** Event filter function */
	ShouldRecordEventFunction ShouldRecordEventFunc;

	/**
	* Delegate called when an event Http request completes
	*/
	void EventRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded);

	TSharedPtr<class FHttpRetrySystem::FManager> HttpRetryManager;
	FHttpRetrySystem::FRetryDomainsPtr RetryServers;
	/** Http headers to add to requests */
	TMap<FString, FString> HttpHeaders;

	FAnalyticsUserAgentCache UserAgentCache;

	FOnPreAnalyticsEventProcessed OnPreAnalyticsEventProcessedDelegate;
	FOnAnalyticsEventQueued OnAnalyticsEventQueuedDelegate;
};

TSharedPtr<IAnalyticsProviderET> FAnalyticsET::CreateAnalyticsProvider(const Config& ConfigValues) const
{
#ifdef DISABLE_ANALYTICS_PROVIDER

	UE_LOGF(LogAnalytics, Warning, "CreateAnalyticsProvider is disabled in this configuration.");
	return NULL;
#else
	// If we didn't have a proper APIKey, return NULL
	if (ConfigValues.APIKeyET.IsEmpty())
	{
		UE_LOGF(LogAnalytics, Warning, "CreateAnalyticsProvider config not contain required parameter %ls", *Config::GetKeyNameForAPIKey());
		return NULL;
	}
	LLM_SCOPE_BYNAME(TEXT("Analytics"));
	return MakeShared<FAnalyticsProviderET>(ConfigValues);
#endif
}

/**
 * Perform any initialization.
 */
FAnalyticsProviderET::FAnalyticsProviderET(const FAnalyticsET::Config& ConfigValues)
	: bSessionInProgress(false)
	, Config(ConfigValues)
	, FlushIntervalSec(ConfigValues.FlushIntervalSec < 0 ? DefaultFlushIntervalSec : ConfigValues.FlushIntervalSec)
	, bShouldCacheEvents(true)
	, NextEventFlushTime(FPlatformTime::Seconds() + FlushIntervalSec)
	, bInDestructor(false)
	// avoid preallocating space if we are using the legacy protocol.
	, EventCache(ConfigValues.MaximumPayloadSize, ConfigValues.UseLegacyProtocol ? 0 : ConfigValues.PreallocatedPayloadSize)
{
	if (Config.APIKeyET.IsEmpty())
	{
		UE_LOGF(LogAnalytics, Fatal, "AnalyticsET: APIKey (%ls) cannot be empty!", *Config.APIKeyET);
	}

	// Set the number of retries to the number of retry URLs that have been passed in.
	uint32 RetryLimitCount = ConfigValues.AltAPIServersET.Num();

	HttpRetryManager = MakeShared<FHttpRetrySystem::FManager>(
		FHttpRetrySystem::FRetryLimitCountSetting(RetryLimitCount),
		FHttpRetrySystem::FRetryTimeoutRelativeSecondsSetting()
		);

	// If we have retry domains defined, insert the default domain into the list
	if (RetryLimitCount > 0)
	{
		TArray<FString> TmpAltAPIServers = ConfigValues.AltAPIServersET;

		FString DefaultUrlDomain = FPlatformHttp::GetUrlDomain(Config.APIServerET);
		if (!TmpAltAPIServers.Contains(DefaultUrlDomain))
		{
			TmpAltAPIServers.Insert(DefaultUrlDomain, 0);
		}

		RetryServers = MakeShared<FHttpRetrySystem::FRetryDomains, ESPMode::ThreadSafe>(MoveTemp(TmpAltAPIServers));
	}

	const bool bTestingMode = FParse::Param(FCommandLine::Get(), TEXT("TELEMETRYTESTING"));
	if (bTestingMode)
	{
		UE_SET_LOG_VERBOSITY(LogAnalytics, VeryVerbose);
		bShouldCacheEvents = false;
	}

	// force very verbose logging if we are force-disabling events.
	bool bForceDisableCaching = FParse::Param(FCommandLine::Get(), TEXT("ANALYTICSDISABLECACHING"));
	if (bForceDisableCaching)
	{
		UE_SET_LOG_VERBOSITY(LogAnalytics, VeryVerbose);
		bShouldCacheEvents = false;
	}

	UE_LOGF(LogAnalytics, Verbose, "[%ls] Initializing ET Analytics provider", *Config.APIKeyET);

	// default to FEngineVersion::Current() if one is not provided, append FEngineVersion::Current() otherwise.
	FString ConfigAppVersion = ConfigValues.AppVersionET;
	// Allow the cmdline to force a specific AppVersion so it can be set dynamically.
	FParse::Value(FCommandLine::Get(), TEXT("ANALYTICSAPPVERSION="), ConfigAppVersion, false);
	Config.AppVersionET = ConfigAppVersion.IsEmpty()
		? FString(FApp::GetBuildVersion())
		: ConfigAppVersion.Replace(TEXT("%VERSION%"), FApp::GetBuildVersion(), ESearchCase::CaseSensitive);

	if (Config.APIEndpointET.IsEmpty())
	{
		Config.APIEndpointET = FAnalyticsET::Config::GetDefaultAPIEndpoint();
	}

	if (Config.APIServerET.IsEmpty())
	{
		UE_LOGF(LogAnalytics, Warning, "AnalyticsET: APIServerET is empty for APIKey (%ls), creating as a NULL provider!", *Config.APIKeyET);
	}

#if !UE_HTTP_SUPPORT_UNIX_SOCKET
	if (!Config.APIUnixSocketPathET.IsEmpty())
	{
		UE_LOGF(LogAnalytics, Warning, "[%ls] Specified UnixSocketPath '%ls' but that is not supported on this platform", *Config.APIKeyET, *Config.APIUnixSocketPathET);
	}
#endif //UE_HTTP_SUPPORT_UNIX_SOCKET

	// only need these if we are using the data router protocol.
	if (!Config.UseLegacyProtocol)
	{
		Config.AppEnvironment = ConfigValues.AppEnvironment.IsEmpty()
			? FAnalyticsET::Config::GetDefaultAppEnvironment()
			: ConfigValues.AppEnvironment;
		Config.UploadType = ConfigValues.UploadType.IsEmpty()
			? FAnalyticsET::Config::GetDefaultUploadType()
			: ConfigValues.UploadType;
	}

	// see if there is a cmdline supplied UserID.
#if !UE_BUILD_SHIPPING
	FString ConfigUserID;
	if (FParse::Value(FCommandLine::Get(), TEXT("ANALYTICSUSERID="), ConfigUserID, false))
	{
		SetUserID(ConfigUserID);
	}
#endif // !UE_BUILD_SHIPPING
}

bool FAnalyticsProviderET::Tick(float DeltaSeconds)
{
	LLM_SCOPE_BYNAME(TEXT("Analytics"));
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FAnalyticsProviderET_Tick);

	// hold a lock the entire time here because we're making several calls to the event cache that we need to be consistent when we decide to flush.
	// With more care, we can likely avoid holding this lock the entire time.
	FAnalyticsProviderETEventCache::Lock EventCacheLock(EventCache);

	// Countdown to flush
	const double Now = FPlatformTime::Seconds();

	// Never tick-flush more than one provider in a single frame. There's non-trivial overhead to flushing events.
	// On servers where there may be dozens of provider instances, this will spread out the cost a bit.
	// If caching is disabled, we still want events to be flushed immediately, so we are only guarding the flush calls from tick,
	// any other calls to flush are allowed to happen in the same frame.
	static uint64 LastFrameCounterFlushed = 0;

	const bool bHadFlushesQueued = EventCache.HasFlushesQueued();
	const bool bShouldFlush = bHadFlushesQueued || (EventCache.CanFlush() && Now >= NextEventFlushTime);

	if (bShouldFlush)
	{
		if (GFrameCounter == LastFrameCounterFlushed && AnalyticsProviderETCvars::PreventMultipleFlushesInOneFrame)
		{
			UE_LOGF(LogAnalytics, Verbose, "[%ls] Tried to flush, but another analytics provider has already flushed this frame. Deferring until next frame.", *Config.APIKeyET);
		}
		else
		{
			// Just flush one payload, even if we may have more than one queued.
			FlushEventsOnce();
			LastFrameCounterFlushed = GFrameCounter;
			// If we aren't flushing up a previous queued payload, then this was a regular interval flush, so we need to reset the timer.
			// try to keep on the same cadence when flushing, since we could miss our window by several frames.
			if (!bHadFlushesQueued && Now >= NextEventFlushTime)
			{
				const double Multiplier = FMath::Floor((Now - NextEventFlushTime) / FlushIntervalSec) + 1.;
				NextEventFlushTime += Multiplier * FlushIntervalSec;
			}
		}
	}
	return true;
}

FAnalyticsProviderET::~FAnalyticsProviderET()
{
	bInDestructor = true;
	EndSession();
}

bool FAnalyticsProviderET::StartSession(FString InSessionID, const TArray<FAnalyticsEventAttribute>& Attributes)
{
	LLM_SCOPE_BYNAME(TEXT("Analytics"));
	UE_LOGF(LogAnalytics, Display, "[%ls] AnalyticsET::StartSession ( APIServer = %ls%ls. AppVersion = %ls )", *Config.APIKeyET, *Config.APIServerET, *Config.APIEndpointET, *Config.AppVersionET);

	// end/flush previous session before staring new one
	if (bSessionInProgress)
	{
		EndSession();
	}
	SessionID = MoveTemp(InSessionID);
	// always ensure we send a few specific attributes on session start.
	TArray<FAnalyticsEventAttribute> AttributesWithPlatform = Attributes;
	AttributesWithPlatform.Emplace(TEXT("Platform"), FString(FPlatformProperties::IniPlatformName()));

	RecordEvent(TEXT("SessionStart"), AttributesWithPlatform);
	bSessionInProgress = true;
	return bSessionInProgress;
}

/**
 * End capturing stats and queue the upload
 */
void FAnalyticsProviderET::EndSession()
{
	LLM_SCOPE_BYNAME(TEXT("Analytics"));
	if (bSessionInProgress)
	{
		RecordEvent(TEXT("SessionEnd"), TArray<FAnalyticsEventAttribute>());
		UE_LOGF(LogAnalytics, Display, "[%ls] AnalyticsET::EndSession", *Config.APIKeyET);
	}

	FlushEvents();
	SessionID.Empty();

	bSessionInProgress = false;
}

TSharedRef<IHttpRequest, ESPMode::ThreadSafe> FAnalyticsProviderET::CreateRequest(EAnalyticsRecordEventMode Mode)
{
	LLM_SCOPE_BYNAME(TEXT("Analytics"));
	if (!ensure(FModuleManager::Get().IsModuleLoaded("HTTP")))
	{
		UE_LOGF(LogAnalytics, Display, "[%ls] ET Analytics provider tried to create a new HTTP request when HTTP was shutdown", *Config.APIKeyET);
	}

	// TODO add config values for retries, for now, using default
	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = HttpRetryManager->CreateRequest(FHttpRetrySystem::FRetryLimitCountSetting(),
		FHttpRetrySystem::FRetryTimeoutRelativeSecondsSetting(),
		FHttpRetrySystem::FRetryResponseCodes(),
		FHttpRetrySystem::FRetryVerbs(),
		RetryServers,
		FHttpRetrySystem::FRetryLimitCountSetting(),
		FHttpRetrySystem::FExponentialBackoffCurve());
	for (const TPair<FString, FString>& HttpHeader : HttpHeaders)
	{
		HttpRequest->SetHeader(HttpHeader.Key, HttpHeader.Value);
	}

	if (Mode == EAnalyticsRecordEventMode::Immediate)
	{
		HttpRequest->SetOption(HttpRequestOptions::RequestMode, LexToString(EHttpRequestMode::ImmediateRequest));
	}

	return HttpRequest;
}

void FAnalyticsProviderET::FlushEvents()
{
	UE_AUTORTFM_ONCOMMIT(this)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FAnalyticsProviderET_FlushEvents);
		// Warn if this takes more than 2 ms
		SCOPE_TIME_GUARD_MS(TEXT("FAnalyticsProviderET::FlushEvents"), 2);

		// keep flushing until the event cache has cleared its queue.
		while (EventCache.CanFlush())
		{
			FlushEventsOnce();
		}
	};
}

void FAnalyticsProviderET::ExecuteRequest(TArray<uint8>& Payload, OUT int32& PayloadSize, OUT int32& EventCount, EAnalyticsRecordEventMode Mode)
{
	EventCount = 0;
	PayloadSize = 0;

	// UrlEncode NOTE: need to concatenate everything
	FString URLPath = Config.APIEndpointET;
			URLPath += TEXT("?SessionID=") + FPlatformHttp::UrlEncode(SessionID);
			URLPath += TEXT("&AppID=") + FPlatformHttp::UrlEncode(Config.APIKeyET);
			URLPath += TEXT("&AppVersion=") + FPlatformHttp::UrlEncode(Config.AppVersionET);
			URLPath += TEXT("&UserID=") + FPlatformHttp::UrlEncode(UserID);
			URLPath += TEXT("&AppEnvironment=") + FPlatformHttp::UrlEncode(Config.AppEnvironment);
			URLPath += TEXT("&UploadType=") + FPlatformHttp::UrlEncode(Config.UploadType);
	PayloadSize = URLPath.Len() + Payload.Num();

	// Recreate the URLPath for logging because we do not want to escape the parameters when logging.
	// We cannot simply UrlEncode the entire Path after logging it because UrlEncode(Params) != UrlEncode(Param1) & UrlEncode(Param2) ...
	UE_LOGF(LogAnalytics, VeryVerbose, "[%ls] AnalyticsET URL:%ls?SessionID=%ls&AppID=%ls&AppVersion=%ls&UserID=%ls&AppEnvironment=%ls&UploadType=%ls. Payload:%.*s",
		*Config.APIKeyET,
		*Config.APIEndpointET,
		*SessionID,
		*Config.APIKeyET,
		*Config.AppVersionET,
		*UserID,
		*Config.AppEnvironment,
		*Config.UploadType,
		Payload.Num(), reinterpret_cast<FGenericPlatformTypes::UTF8CHAR*>(Payload.GetData()));

	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FlushEventsHttpRequest);

		// Create/send Http request for an event
		TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = CreateRequest(Mode);
		HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json; charset=utf-8"));
		// Want to avoid putting the project name into the User-Agent, because for some apps (like the editor), the project name is private info.
		// The analytics User-Agent uses the default User-Agent, but with project name removed.
		HttpRequest->SetHeader(TEXT("User-Agent"), UserAgentCache.GetUserAgent());
		HttpRequest->SetURL(Config.APIServerET / URLPath);
		HttpRequest->SetVerb(TEXT("POST"));
		HttpRequest->SetContent(MoveTemp(Payload));

#if UE_HTTP_SUPPORT_UNIX_SOCKET
		if (!Config.APIUnixSocketPathET.IsEmpty())
		{
			HttpRequest->SetOption(HttpRequestOptions::UnixSocketPath, Config.APIUnixSocketPathET);
		}
#endif //UE_HTTP_SUPPORT_UNIX_SOCKET

		// Don't set a response callback if we are in our destructor, as the instance will no longer be there to call.
		if (!bInDestructor)
		{
			HttpRequest->OnProcessRequestComplete().BindSP(this, &FAnalyticsProviderET::EventRequestComplete);
		}

		OnPreAnalyticsEventProcessedDelegate.Broadcast(HttpRequest);

		HttpRequest->ProcessRequest();
	}
}

void FAnalyticsProviderET::SendImmediately(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes)
{
	LLM_SCOPE_BYNAME(TEXT("Analytics"));

	if (ensure(FModuleManager::Get().IsModuleLoaded("HTTP")))
	{
		TArray<uint8> Payload = EventCache.CreateImmediatePayload(EventName, Attributes);

		int32 PayloadSize = 0;
		int32 EventCount = 0;
		ExecuteRequest(Payload, PayloadSize, EventCount, EAnalyticsRecordEventMode::Immediate);
	}
	else
	{
		// If the HTTP module is not loaded yet for some reason, we fallback to the Cached system so we won't lose this event.
		// This should not happen, but better safe than sorry.
		RecordEvent(CopyTemp(EventName), Attributes, EAnalyticsRecordEventMode::Cached);
	}
}

void FAnalyticsProviderET::FlushEventsOnce()
{
	// FlushEventsOnce cannot be rolled back, so it must not occur inside an AutoRTFM transaction.
	if (!ensure(!AutoRTFM::IsClosed()))
	{
		return;
	}

	LLM_SCOPE_BYNAME(TEXT("Analytics"));
	// Make sure we don't try to flush too many times. When we are not caching events it's possible this can be called when there are no events in the array.
	if (!EventCache.CanFlush())
	{
		return;
	}

	ANALYTICS_FLUSH_TRACKING_BEGIN();
	int EventCount = 0;
	int PayloadSize = 0;

	if(ensure(FModuleManager::Get().IsModuleLoaded("HTTP")))
	{
		TArray<uint8> Payload = EventCache.FlushCacheUTF8();
		
		ExecuteRequest(Payload, PayloadSize, EventCount, EAnalyticsRecordEventMode::Cached);
	}
	ANALYTICS_FLUSH_TRACKING_END(PayloadSize, EventCount);
}

void FAnalyticsProviderET::SetAppID(FString&& InAppID)
{
	if (Config.APIKeyET != InAppID)
	{
		// Flush any cached events that would be using the old AppID.
		FlushEvents();
		Config.APIKeyET = MoveTemp(InAppID);
	}
}

void FAnalyticsProviderET::SetAppVersion(FString&& InAppVersion)
{
	// make sure to do the version replacement if the given string is parameterized.
	InAppVersion = InAppVersion.IsEmpty()
		? FString(FApp::GetBuildVersion())
		: InAppVersion.Replace(TEXT("%VERSION%"), FApp::GetBuildVersion(), ESearchCase::CaseSensitive);

	if (Config.AppVersionET != InAppVersion)
	{
		UE_LOGF(LogAnalytics, Log, "[%ls] Updating AppVersion to %ls from old value of %ls", *Config.APIKeyET, *InAppVersion, *Config.AppVersionET);
		// Flush any cached events that would be using the old AppVersion.
		FlushEvents();
		Config.AppVersionET = MoveTemp(InAppVersion);
	}
}

void FAnalyticsProviderET::SetUserID(const FString& InUserID)
{
	// command-line specified user ID overrides all attempts to reset it.
	if (!FParse::Value(FCommandLine::Get(), TEXT("ANALYTICSUSERID="), UserID, false))
	{
		UE_LOGF(LogAnalytics, Log, "[%ls] SetUserId %ls", *Config.APIKeyET, *InUserID);
		// Flush any cached events that would be using the old UserID.
		FlushEvents();
		UserID = InUserID;
	}
	else if (UserID != InUserID)
	{
		UE_LOGF(LogAnalytics, Log, "[%ls] Overriding SetUserId %ls with cmdline UserId of %ls.", *Config.APIKeyET, *InUserID, *UserID);
	}
}

FString FAnalyticsProviderET::GetUserID() const
{
	return UserID;
}

FString FAnalyticsProviderET::GetSessionID() const
{
	return SessionID;
}

bool FAnalyticsProviderET::SetSessionID(const FString& InSessionID)
{
	if (SessionID != InSessionID)
	{
		// Flush any cached events that would be using the old SessionID.
		FlushEvents();
		SessionID = InSessionID;
		UE_LOGF(LogAnalytics, Log, "[%ls] Forcing SessionID to %ls.", *Config.APIKeyET, *SessionID);
	}
	return true;
}

bool FAnalyticsProviderET::ShouldRecordEvent(const FString& EventName) const
{
	return !IsActingAsNullProvider() && (!ShouldRecordEventFunc || ShouldRecordEventFunc(*this, EventName));
}

void FAnalyticsProviderET::RecordEvent(FString&& EventName, const TArray<FAnalyticsEventAttribute>& Attributes)
{
	LLM_SCOPE_BYNAME(TEXT("Analytics"));

	RecordEvent(MoveTemp(EventName), Attributes, EAnalyticsRecordEventMode::Cached);
}

void FAnalyticsProviderET::RecordEvent(FString&& EventName, const TArray<FAnalyticsEventAttribute>& Attributes, EAnalyticsRecordEventMode Mode)
{
	// let higher level code filter the decision of whether to send the event
	if (ShouldRecordEvent(EventName))
	{
		bool bQueueEvent = true;
		const FAnalyticEventQueuedInfo AnalyticEventInfo =
		{
			Mode,
			EventName,
			Attributes
		};

		OnAnalyticsEventQueuedDelegate.Broadcast(bQueueEvent, AnalyticEventInfo);

		if (bQueueEvent)
		{
			switch (Mode)
			{
				case EAnalyticsRecordEventMode::Cached:
				{
					// fire any callbacks
					for (const auto& Cb : EventRecordedCallbacks)
					{
						// we no longer track if the event was Json, each attribute does.
						Cb(EventName, Attributes, false);
					}

					if (!Config.UseLegacyProtocol)
					{
						EventCache.AddToCache(MoveTemp(EventName), Attributes);

						// if we aren't caching events, flush immediately. This is really only for debugging as it will significantly affect bandwidth.
						if (!bShouldCacheEvents)
						{
							FlushEvents();
						}
					}
					else
					{
						FlushEventLegacy(EventName, Attributes);
					}
				}
				break;

				case EAnalyticsRecordEventMode::Immediate:
				{
					SendImmediately(MoveTemp(EventName), Attributes);
				}
				break;

				default:
					checkf(false, TEXT("Please, implement new modes here"));
					break;
			}
		}
		else
		{
			UE_LOGF(LogAnalytics, Verbose, "Dropping Event %ls due to OnAnalyticsEventQueuedDelegate request.", *EventName);
		}
	}
	else
	{
		UE_LOGF(LogAnalytics, Verbose, "Ignoring event named '%ls' due to ShouldRecordEvent check", *EventName);
	}
}

void FAnalyticsProviderET::SetDefaultEventAttributes(TArray<FAnalyticsEventAttribute>&& Attributes)
{
	EventCache.SetDefaultAttributes(MoveTemp(Attributes));
}

TArray<FAnalyticsEventAttribute> FAnalyticsProviderET::GetDefaultEventAttributesSafe() const
{
	return EventCache.GetDefaultAttributes();
}

int32 FAnalyticsProviderET::GetDefaultEventAttributeCount() const
{
	return EventCache.GetDefaultAttributeCount();
}


FAnalyticsEventAttribute FAnalyticsProviderET::GetDefaultEventAttribute(int AttributeIndex) const
{
	return EventCache.GetDefaultAttribute(AttributeIndex);
}

void FAnalyticsProviderET::SetEventCallback(const OnEventRecorded& Callback)
{
	EventRecordedCallbacks.Add(Callback);
}

void FAnalyticsProviderET::EventRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool)
{
	// process responses
	bool bEventsDelivered = false;
	if (HttpResponse.IsValid())
	{
		UE_LOGF(LogAnalytics, VeryVerbose, "[%ls] ET response for [%ls]. Code: %d. Payload: %ls", *Config.APIKeyET, *HttpRequest->GetURL(), HttpResponse->GetResponseCode(), *HttpResponse->GetContentAsString());
		if (EHttpResponseCodes::IsOk(HttpResponse->GetResponseCode()))
		{
			bEventsDelivered = true;
		}
	}
	else
	{
		UE_LOGF(LogAnalytics, VeryVerbose, "[%ls] ET response for [%ls]. No response", *Config.APIKeyET, *HttpRequest->GetURL());
	}
}

void FAnalyticsProviderET::SetUrlDomain(const FString& Domain, const TArray<FString>& AltDomains)
{
	// See if anything is actually changing before going through the work to flush and reset the URLs.
	if (Config.APIServerET == Domain && Config.AltAPIServersET == AltDomains)
	{
		return;
	}
	LLM_SCOPE_BYNAME(TEXT("Analytics"));

	// flush existing events before changing URL domains.
	FlushEvents();

	Config.APIServerET = Domain;
	Config.AltAPIServersET = AltDomains;

	// Set the number of retries to the number of retry URLs that have been passed in.
	uint32 RetryLimitCount = AltDomains.Num();

	HttpRetryManager->SetDefaultRetryLimit(RetryLimitCount);

	TArray<FString> TmpAltAPIServers = AltDomains;

	// If we have retry domains defined, insert the default domain into the list
	if (RetryLimitCount > 0)
	{
		FString DefaultUrlDomain = FPlatformHttp::GetUrlDomain(Config.APIServerET);
		if (!TmpAltAPIServers.Contains(DefaultUrlDomain))
		{
			TmpAltAPIServers.Insert(DefaultUrlDomain, 0);
		}

		RetryServers = MakeShared<FHttpRetrySystem::FRetryDomains, ESPMode::ThreadSafe>(MoveTemp(TmpAltAPIServers));
	}
	else
	{
		RetryServers.Reset();
	}

	if (Config.APIServerET.IsEmpty())
	{
		UE_LOGF(LogAnalytics, Warning, "AnalyticsET: APIServerET is empty for APIKey (%ls), converting to a NULL provider!", *Config.APIKeyET);
	}
	else
	{
		UE_LOGF(LogAnalytics, Log, "AnalyticsET: Set APIServerET to %ls", *Config.APIServerET);
	}
}

void FAnalyticsProviderET::SetUrlPath(const FString& Path)
{
	// See if anything is actually changing before going through the work to flush and reset the URLs.
	if (Config.APIEndpointET == Path)
	{
		return;
	}
	LLM_SCOPE_BYNAME(TEXT("Analytics"));

	// flush existing events before changing URL path.
	FlushEvents();

	Config.APIEndpointET = Path;
}

void FAnalyticsProviderET::SetHeader(const FString& HeaderName, const FString& HeaderValue)
{
	LLM_SCOPE_BYNAME(TEXT("Analytics"));
	if (HeaderValue.IsEmpty())
	{
		HttpHeaders.Remove(HeaderName);
	}
	else
	{
		HttpHeaders.Emplace(HeaderName, HeaderValue);
	}
}

void FAnalyticsProviderET::BlockUntilFlushed(float InTimeoutSec)
{
	LLM_SCOPE_BYNAME(TEXT("Analytics"));
	FlushEvents();
	HttpRetryManager->BlockUntilFlushed(InTimeoutSec);
}

void FAnalyticsProviderET::SetShouldRecordEventFunc(const ShouldRecordEventFunction& InShouldRecordEventFunc)
{
	ShouldRecordEventFunc = InShouldRecordEventFunc;
}

static inline void AnalyticsProviderETFlushEventLegacyHelper(FString& EventParams, int PayloadNdx, const FAnalyticsEventAttribute& Attribute)
{
	EventParams += FString::Printf(TEXT("&AttributeName%d=%s&AttributeValue%d=%s"),
		PayloadNdx,
		*FPlatformHttp::UrlEncode(Attribute.GetName()),
		PayloadNdx,
		*FPlatformHttp::UrlEncode(Attribute.GetValue()));
}

void FAnalyticsProviderET::FlushEventLegacy(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes)
{
	// If we are running transactionally, postpone the HTTP request until our transaction is committed.
	// We need to explicitly copy the input arguments since they are passed in by reference.
	UE_AUTORTFM_ONCOMMIT(this, EventName = EventName, Attributes = Attributes)
	{
		// this is a legacy pathway that doesn't accept batch payloads of cached data. We'll just send one request for each event, which will be slow for a large batch of requests at once.
		if (ensure(FModuleManager::Get().IsModuleLoaded("HTTP")))
		{
			LLM_SCOPE_BYNAME(TEXT("Analytics"));
			// first generate a payload from the eventand attributes
			FString EventParams;
			int PayloadNdx = 0;
			for (int DefaultNdx = 0, NumDefaults = EventCache.GetDefaultAttributeCount(); DefaultNdx < NumDefaults; ++DefaultNdx)
			{
				AnalyticsProviderETFlushEventLegacyHelper(EventParams, PayloadNdx, EventCache.GetDefaultAttribute(DefaultNdx));
				++PayloadNdx;
			}
			for (int AttrNdx = 0; AttrNdx < Attributes.Num(); ++AttrNdx)
			{
				AnalyticsProviderETFlushEventLegacyHelper(EventParams, PayloadNdx, Attributes[AttrNdx]);
				++PayloadNdx;
			}

			// log out the un-encoded values to make reading the log easier.
			UE_LOGF(LogAnalytics, VeryVerbose, "[%ls] AnalyticsET URL:SendEvent.1?SessionID=%ls&AppID=%ls&AppVersion=%ls&UserID=%ls&EventName=%ls%ls",
				*Config.APIKeyET,
				*SessionID,
				*Config.APIKeyET,
				*Config.AppVersionET,
				*UserID,
				*EventName,
				*EventParams);

			// Create/send Http request for an event
			TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = CreateRequest();
			HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("text/plain"));

			// Don't need to URL encode the APIServer or the EventParams, which are already encoded, and contain parameter separaters that we DON'T want encoded.
			FString URLPath = Config.APIServerET + Config.APIEndpointET;
			URLPath += TEXT("SendEvent.1?SessionID=") + FPlatformHttp::UrlEncode(SessionID);
			URLPath += TEXT("&AppID=") + FPlatformHttp::UrlEncode(Config.APIKeyET);
			URLPath += TEXT("&AppVersion=") + FPlatformHttp::UrlEncode(Config.AppVersionET);
			URLPath += TEXT("&UserID=") + FPlatformHttp::UrlEncode(UserID);
			URLPath += TEXT("&EventName=") + FPlatformHttp::UrlEncode(EventName);
			URLPath += EventParams;
			HttpRequest->SetURL(URLPath);
			HttpRequest->SetVerb(TEXT("GET"));

#if UE_HTTP_SUPPORT_UNIX_SOCKET
			if (!Config.APIUnixSocketPathET.IsEmpty())
			{
				HttpRequest->SetOption(HttpRequestOptions::UnixSocketPath, Config.APIUnixSocketPathET);
			}
#endif //UE_HTTP_SUPPORT_UNIX_SOCKET

			if (!bInDestructor)
			{
				HttpRequest->OnProcessRequestComplete().BindSP(this, &FAnalyticsProviderET::EventRequestComplete);
			}

			OnPreAnalyticsEventProcessedDelegate.Broadcast(HttpRequest);

			HttpRequest->ProcessRequest();
		}
	};
}
