// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformProcess.h"
#include "Interfaces/IAnalyticsPropertyStore.h"
#include "Misc/AutomationTest.h"
#include "AnalyticsSessionSummaryManager.h"
#include "IAnalyticsSessionSummarySender.h"
#include "Misc/Paths.h"

//#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAnalyticsSessionSummaryManagerAutomationTest, "System.Analytics.SessionSummaryManager", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter | EAutomationTestFlags::MediumPriority)

// A dummy sender to observe what the manager would send.
class FDummyAnalyticsSummarySender : public IAnalyticsSessionSummarySender
{
public:
	virtual bool SendSessionSummary(const FString& InUserId, const FString& InAppId, const FString& InAppVersion, const FString& InSessionId, const TArray<FAnalyticsEventAttribute>& InProperties) override
	{
		UserId = InUserId;
		AppId = InAppId;
		AppVersion = InAppVersion;
		SessionId = InSessionId;
		Properties = InProperties;
		bReportSent = true;
		return true;
	}

	bool bReportSent = false;
	FString UserId;
	FString AppId;
	FString AppVersion;
	FString SessionId;
	TArray<FAnalyticsEventAttribute> Properties;
};

bool FAnalyticsSessionSummaryManagerAutomationTest::RunTest(const FString& Parameters)
{
	const FString PrincipalProcessName       = "DummyPrincipal";
	const FString SubsidiaryProcessName      = "DummySubsidiary";
	const FString ProcessGroupId             = FGuid::NewGuid().ToString(EGuidFormats::Digits);
	const FString PrincipalProcessUserId     = "DummyUserId";
	const FString PrincipalProcessAppId      = "DummyAppId";
	const FString PrincipalProcessAppVersion = "DummyAppVersion";
	const FString PrincipalProcessSessionId  = FGuid::NewGuid().ToString(EGuidFormats::DigitsWithHyphensInBraces);
	const FString SavedDir                   = FPaths::AutomationTransientDir() / TEXT("AnalyticsTest");

	IFileManager::Get().MakeDirectory(*SavedDir, /*Tree*/true);

	// Properties automatically added by the manager.
	const TAnalyticsProperty<bool>    DelayedSendProp = TEXT("DelayedSend");
	const TAnalyticsProperty<FString> SessionIdProp   = TEXT("SessionId");
	const TAnalyticsProperty<FString> SentFromProp    = TEXT("SentFrom");

	const TAnalyticsProperty<int32> Prop1 = TEXT("Prop1");
	const int32 Prop1Value = 42;

	const TAnalyticsProperty<int64> Prop2 = TEXT("Prop2");
	const int32 Prop2Value = 84;

	//
	// Test the basic flow simulating a principal process running alone (Like Editor without CRC).
	//
	{
		// Create the manager.
		FAnalyticsSessionSummaryManager PrincipalManager(PrincipalProcessName, ProcessGroupId, PrincipalProcessUserId, PrincipalProcessAppId, PrincipalProcessAppVersion, PrincipalProcessSessionId, SavedDir);

		// Set the summary sender to observe what the manager is going to send.
		TSharedPtr<FDummyAnalyticsSummarySender> SummarySender = MakeShared<FDummyAnalyticsSummarySender>();
		PrincipalManager.SetSender(StaticCastSharedPtr<IAnalyticsSessionSummarySender>(SummarySender));

		// Create a key/value database to store the session summary properties.
		TSharedPtr<IAnalyticsPropertyStore> PropertyStore = PrincipalManager.MakeStore(/*Capacity*/1024);
		check(PropertyStore);

		// Store a property, flush and release the store (so that it can be reloaded).
		Prop1.Set(PropertyStore.Get(), Prop1Value);
		PropertyStore->Flush();
		PropertyStore.Reset();

		// Shutdown the manager. This should emits the summary.
		PrincipalManager.Shutdown();

		// Ensure the manager used the correct config to send.
		check(SummarySender->UserId     == PrincipalProcessUserId);
		check(SummarySender->AppId      == PrincipalProcessAppId);
		check(SummarySender->AppVersion == PrincipalProcessAppVersion);
		check(SummarySender->SessionId  == PrincipalProcessSessionId);

		// Check the automatically embedded keys.
		check(SummarySender->Properties.FindByPredicate([&](const FAnalyticsEventAttribute& Attr) { return Attr.GetName() == DelayedSendProp.Key; })->GetValue() == LexToString(false));
		check(SummarySender->Properties.FindByPredicate([&](const FAnalyticsEventAttribute& Attr) { return Attr.GetName() == SessionIdProp.Key; })->GetValue()   == PrincipalProcessSessionId);
		check(SummarySender->Properties.FindByPredicate([&](const FAnalyticsEventAttribute& Attr) { return Attr.GetName() == SentFromProp.Key; })->GetValue()    == PrincipalProcessName);

		// Check the custom property.
		check(SummarySender->Properties.FindByPredicate([&](const FAnalyticsEventAttribute& Attr) { return Attr.GetName() == Prop1.Key; })->GetValue() == LexToString(Prop1Value));
	}

	//
	// Test discarding the session in a principal process.
	//
	{
		// Create the managers.
		FAnalyticsSessionSummaryManager PrincipalManager(PrincipalProcessName, ProcessGroupId, PrincipalProcessUserId, PrincipalProcessAppId, PrincipalProcessAppVersion, PrincipalProcessSessionId, SavedDir);
		
		// Set the summary sender to observe what the managers are going to send.
		TSharedPtr<FDummyAnalyticsSummarySender> PrincipalSender = MakeShared<FDummyAnalyticsSummarySender>();
		PrincipalManager.SetSender(StaticCastSharedPtr<IAnalyticsSessionSummarySender>(PrincipalSender));

		// Create a key/value database to store the session summary properties.
		TSharedPtr<IAnalyticsPropertyStore> PrincipalStore = PrincipalManager.MakeStore(/*Capacity*/1024);
		check(PrincipalStore);

		// Store a property.
		Prop1.Set(PrincipalStore.Get(), Prop1Value);

		// Flush and release the store.
		PrincipalStore->Flush();
		PrincipalStore.Reset();

		// Discard the summary
		PrincipalManager.Shutdown(/*bDiscard*/true);

		// Ensure no report was sent.
		check(!PrincipalSender->bReportSent);

		// Ensure the file created by the process was deleted.
		FString PropertyStorePathname = SavedDir / FString::Printf(TEXT("%s_%d_%d_%s"), *ProcessGroupId, FPlatformProcess::GetCurrentProcessId(), FPlatformProcess::GetCurrentProcessId(), *PrincipalProcessName);
		check(!IFileManager::Get().FileExists(*PropertyStorePathname));
	}

	return true;
}

//#endif WITH_DEV_AUTOMATION_TESTS
