// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnalyticsProviderConfigurationDelegate.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Interfaces/IAnalyticsProviderModule.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

class IAnalyticsProvider;
class IAnalyticsProviderET;

/**
 *  Public implementation of EpicGames.MCP.AnalyticsProvider
 */
class FAnalyticsET : public IAnalyticsProviderModule
{
	//--------------------------------------------------------------------------
	// Module functionality
	//--------------------------------------------------------------------------
public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline FAnalyticsET& Get()
	{
		return FModuleManager::LoadModuleChecked< FAnalyticsET >( "AnalyticsET" );
	}

	//--------------------------------------------------------------------------
	// Configuration functionality
	//--------------------------------------------------------------------------
public:
	/**
	 * Defines required configuration values for ET analytics provider. 
	 * APIKeyET MUST be set.
	 * Set APIServerET to an empty string to create a "NULL" analytics provider that will be a valid instance but will suppress sending any events.
	*/
	struct Config
	{
		/** ET APIKey - Get from your account manager */
		FString APIKeyET;
		/** ET API Server - Base URL to send events. Set this to an empty string to essentially create a NULL analytics provider that will be non-null, but won't actually send events. */
		FString APIServerET;
		/** ET API Endpoint - This is the API endpoint for the provider. */
		FString APIEndpointET;
		/** ET API UnixSocketPath - This is the UnixSocketPath to use as the target for the API endpoint for the provider. */
		FString APIUnixSocketPathET;
		/** ET Alt API Servers - Base URLs to send events on retry. */
		TArray<FString> AltAPIServersET;
		/** 
		 * AppVersion - defines the app version passed to the provider. By default this will be FEngineVersion::Current(), but you can supply your own. 
		 * As a convenience, you can use -AnalyticsAppVersion=XXX to force the AppVersion to a specific value. Useful for playtest etc where you want to define a specific version string dynamically.
		 * If you supply your own Version string, occurrences of "%VERSION%" are replaced with FEngineVersion::Current(). ie, -AnalyticsAppVersion=MyCustomID-%VERSION%.
		 */
		FString AppVersionET;
		/** When true, sends events using the legacy ET protocol that passes all attributes as URL parameters. Defaults to false. */
		bool UseLegacyProtocol = false;
		/** When true (default), events are dropped if flush fails */
		bool bDropEventsOnFlushFailure = true;
		/** The AppEnvironment that the data router should use. Defaults to GetDefaultAppEnvironment. */
		FString AppEnvironment;
		/** The UploadType that the data router should use. Defaults to GetDefaultUploadType. */
		FString UploadType;
		/** Maximum number of retries to attempt. */
		uint32 RetryLimitCount = 0;
		/** Maximum time to elapse before forcing events to be flushed. Use a negative value to use the defaults (60 sec). */
		float FlushIntervalSec = -1.f;
		/** Maximum size a payload can reach before we force a flush of the payload. Use a negative value to use the defaults. See FAnalyticsProviderETEventCache. */
		int32 MaximumPayloadSize = -1;
		/** We preallocate a payload. It defaults to the Maximum configured payload size (see FAnalyticsProviderETEventCache). Use a negative value use the default. See FAnalyticsProviderETEventCache. */
		int32 PreallocatedPayloadSize = -1;

		/** Default ctor to ensure all values have their proper default. */
		Config() = default;
		/** Ctor exposing common configurables . */
		Config(FString InAPIKeyET, FString InAPIServerET, FString InAppVersionET = FString(), bool InUseLegacyProtocol = false, FString InAppEnvironment = FString(), FString InUploadType = FString(), TArray<FString> InAltApiServers = TArray<FString>(), float InFlushIntervalSec = -1.f, int32 InMaximumPayloadSize = -1, int32 InPreallocatedPayloadSize = -1)
			: APIKeyET(MoveTemp(InAPIKeyET))
			, APIServerET(MoveTemp(InAPIServerET))
			, AltAPIServersET(MoveTemp(InAltApiServers))
			, AppVersionET(MoveTemp(InAppVersionET))
			, UseLegacyProtocol(InUseLegacyProtocol)
			, AppEnvironment(MoveTemp(InAppEnvironment))
			, UploadType(MoveTemp(InUploadType))
			, FlushIntervalSec(InFlushIntervalSec)
			, MaximumPayloadSize(InMaximumPayloadSize)
			, PreallocatedPayloadSize(InPreallocatedPayloadSize)
		{}

		/** KeyName required for APIKey configuration. */
		static FString GetKeyNameForAPIKey() { return TEXT("APIKeyET"); }
		/** KeyName required for APIServer configuration. */
		static FString GetKeyNameForAPIServer() { return TEXT("APIServerET"); }
		/** KeyName required for APIEndpoint configuration. */
		static FString GetKeyNameForAPIEndpoint() { return TEXT("APIEndpointET"); }
		/** KeyName required for APIEndpoint configuration. */
		static FString GetKeyNameForAPIUnixSocketPath() { return TEXT("APIUnixSocketPathET"); }
		/** KeyName required for AppVersion configuration. */
		static FString GetKeyNameForAppVersion() { return TEXT("AppVersionET"); }
		/** Optional parameter to use the legacy backend protocol. */
		static FString GetKeyNameForUseLegacyProtocol() { return TEXT("UseLegacyProtocol"); }
		/** For the the data router backend protocol. */
		static FString GetKeyNameForAppEnvironment() { return TEXT("AppEnvironment"); }
		/** For the the data router backend protocol. */
		static FString GetKeyNameForUploadType() { return TEXT("UploadType"); }
		/** Default value if no APIServer configuration is provided. */
		static FString GetDefaultAppEnvironment() { return TEXT("datacollector-binary"); }
		/** Default value if no UploadType is given, and UseDataRouter protocol is specified. */
		static FString GetDefaultUploadType() { return TEXT("eteventstream"); }
		/** Default value if no APIEndpoint is given. */
		static FString GetDefaultAPIEndpoint() { return TEXT("datarouter/api/v1/public/data"); }
	};

	//--------------------------------------------------------------------------
	// provider factory functions
	//--------------------------------------------------------------------------
public:
	/**
	 * IAnalyticsProviderModule interface.
	 * Creates the analytics provider given a configuration delegate.
	 * The keys required exactly match the field names in the Config object. 
	 */
	ANALYTICSET_API virtual TSharedPtr<IAnalyticsProvider> CreateAnalyticsProvider(const FAnalyticsProviderConfigurationDelegate& GetConfigValue) const override;

	/**
	 * Construct an ET analytics provider given a configuration delegate.
	 * The keys required exactly match the field names in the Config object.
	 */
	ANALYTICSET_API virtual TSharedPtr<IAnalyticsProviderET> CreateAnalyticsProviderET(const FAnalyticsProviderConfigurationDelegate& GetConfigValue) const;

	
	/** 
	 * Construct an ET analytics provider directly from a config object.
	 */
	ANALYTICSET_API virtual TSharedPtr<IAnalyticsProviderET> CreateAnalyticsProvider(const Config& ConfigValues) const;

private:
	ANALYTICSET_API virtual void StartupModule() override;
	ANALYTICSET_API virtual void ShutdownModule() override;
};
