// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "IAnalyticsProviderET.h"

using FThreadId = uint32;

class UE_DEPRECATED(5.4, "Use FAnalyticsTracer instead.") FAnalyticsFlowTracker : FNoncopyable
{
public:
	FAnalyticsFlowTracker() {};
	~FAnalyticsFlowTracker() {};

	/** Sets the analytics provider for the flow tracker. */
	ANALYTICSET_API void SetProvider(TSharedPtr<IAnalyticsProvider> AnalyticsProvider);

	/** Begins a new flow tracking session. Will emit Flow and SubFlow events to the specified analytics provider */
	ANALYTICSET_API void StartSession();

	/** Ends all open Flows and SubFlows*/
	ANALYTICSET_API void EndSession();
	
	/** Start a new Flow, the existing flow context will be pushed onto a stack and the new flow will become the current context*/
	ANALYTICSET_API FGuid StartFlow(const FName& FlowName);

	/** End the flow for the current context and pop the stack*/
	ANALYTICSET_API bool EndFlow(bool bSuccess = true, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes = {});

	/** End an existing flow by name */
	ANALYTICSET_API bool EndFlow(const FName& FlowName, bool bSuccess = true, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes = {});

	/** End an bool flow by GUID */
	ANALYTICSET_API bool EndFlow(const FGuid& FlowGuid, bool bSuccess = true, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes = {});

	/** Start a new flow and add it to the current flow context */
	ANALYTICSET_API FGuid StartSubFlow(const FName& SubFlowName);

	/** Start a new flow step and add it to a specific flow context by GUID */
	ANALYTICSET_API FGuid StartSubFlow(const FName& SubFlowName, const FGuid& FlowGuid);

	/** End an existing flow step by name */
	ANALYTICSET_API bool EndSubFlow(const FName& SubFlowName, bool bSuccess = true, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes = {});

	/** End an existing flow step by GUID */
	ANALYTICSET_API bool EndSubFlow(const FGuid& SubFlowGuid, bool bSuccess = true, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes = {});

private:

	struct FSubFlowData
	{
		FName		FlowName;
		FGuid		FlowGuid = FGuid();
		FName		SubFlowName;
		FGuid		SubFlowGuid;
		FDateTime	StartTime = 0;
		FDateTime	EndTime = 0;
		FThreadId	ThreadId = 0;
		double		TimeInSeconds = 0;
		bool		bSuccess = false;
		int32		ScopeDepth;
		TArray<FAnalyticsEventAttribute> AdditionalEventAttributes;
	};


	struct FFlowData
	{
		FName						FlowName = "None";
		FGuid						FlowGuid = FGuid();
		FDateTime					StartTime = 0;
		FDateTime					EndTime = 0;
		FThreadId					ThreadId = 0 ;
		double						TimeInSeconds = 0;
		TArray<FGuid>				SubFlowDataArray;
		TArray<FGuid>				SubFlowDataStack;
	};

	bool EndFlowInternal(const FGuid& FlowGuid, bool bSuccess = true, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes = {});

	FGuid StartSubFlowInternal(const FName& NewSubFlowName, const FGuid& FlowGuid);
	bool EndSubFlowInternal(const FGuid& SubFlowGuid, bool bSuccess = true, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes = {});

	TMap<FName, FGuid>				FlowGuidRegistry;
	TMap<FGuid, FFlowData>			FlowDataRegistry;
	TMap<FName, FGuid>				SubFlowGuidRegistry;
	TMap<FGuid, FSubFlowData>		SubFlowDataRegistry;
	TArray<FGuid>					FlowDataStack;
	FCriticalSection				CriticalSection;
	TSharedPtr<IAnalyticsProvider>	AnalyticsProvider;

	const uint32 FlowSchemaVersion = 4;
	const FString FlowEventName = "Iteration.Flow";

	const uint32 SubFlowSchemaVersion = 4;
	const FString SubFlowEventName = "Iteration.SubFlow";
};


