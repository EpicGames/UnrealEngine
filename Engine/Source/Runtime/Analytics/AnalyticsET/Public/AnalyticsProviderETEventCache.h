// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnalyticsEventAttribute.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "HAL/CriticalSection.h"
#include "Misc/DateTime.h"
#include "Misc/ScopeLock.h"
#include "Templates/Function.h"
#include "Templates/UnrealTemplate.h"

/**
 * Mixin class for Epic Telemetry implementors.
 * The purpose of this class is to support the concept of caching events that are added via the standard RecordEvent API
 * and serializing them into a payload in a Json format compatible with Epic's backend data collectors.
 * The job of transporting these payloads to an external collector (generally expected to be via HTTP) is left to
 * higher level classes to implement.
 *
 * All public APIs in this class are threadsafe. Implemented via crappy critical sections for now, but they are safe. 
 */
class FAnalyticsProviderETEventCache
{
public:
	/** Default ctor.
	 * @param InPreallocatedPayloadSize - size to preallocate the payload buffer to during init and after flushing. 
	 *        If negative, will match the INI-configured MaximumPayloadSize. 
	 * @param MaximumPayloadSize - size before a payload will be queued for flush. 
	 *        This ensures that no payload ever gets too large. See AddToCache() and HasFlushesQueued() for details. 
	 *        If negative, will use INI-configured value: Engine:[AnalyticsProviderETEventCache]MaximumPayloadSize, or 100KB if not configured.
	 */
	ANALYTICSET_API FAnalyticsProviderETEventCache(int32 MaximumPayloadSize = -1, int32 InPreallocatedPayloadSize = -1);

	/** 
	 * Adds a new event to the cache.
	 * If the estimated payload size will increase beyond MaximumPayloadSize then a flush will be queued here. This will make HasFlushesQueued() == true.
	 */
	ANALYTICSET_API void AddToCache(FString EventName, const TArray<FAnalyticsEventAttribute>& Attributes);
	ANALYTICSET_API void AddToCache(FString EventName);

	/**
	 * Sets an array of attributes that will automatically be appended to any event that is sent.
	 * Logical effect is like adding them to all events before calling RecordEvent.
	 * Practically, it is implemented much more efficiently from a storage and allocation perspective.
	 */
	ANALYTICSET_API void SetDefaultAttributes(TArray<FAnalyticsEventAttribute>&& DefaultAttributes);

	/**
	 * @return the current array of default attributes.
	 */
	ANALYTICSET_API TArray<FAnalyticsEventAttribute> GetDefaultAttributes() const;

	/**
	 * @return the number of default attributes are currently being applied.
	 */
	ANALYTICSET_API int32 GetDefaultAttributeCount() const;

	/**
	 * Range checking is not done, similar to TArray. Use GetDefaultAttributeCount() first!
	 * @return one attribute of the default attributes so we don't have to copy the entire attribute array.
	 */
	ANALYTICSET_API FAnalyticsEventAttribute GetDefaultAttribute(int32 AttributeIndex) const;

	/** Flushes the cache as a string. This method is inefficient because we build up the array directly as UTF8. If nothing is cached, returns an empty string. */
	UE_DEPRECATED(4.25, "This method has been deprecated, use FlushCacheUTF8() instead.")
	ANALYTICSET_API FString FlushCache(SIZE_T* OutEventCount = nullptr);

	/** Flushes the cache as a UTF8 char array. Returns a uint8 because that's what IHttpRequest prefers. If nothing is cached, returns an empty array. */
	ANALYTICSET_API TArray<uint8> FlushCacheUTF8();

	/**
	 * Creates a immediate payload with the existing default attributes
	 */
	ANALYTICSET_API TArray<uint8> CreateImmediatePayload(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes) const;

	/**
	* Determines whether we have anything we need to flush, either a queued flush or existing events in the payload.
	*/
	ANALYTICSET_API bool CanFlush() const;

	/**
	 * Lets external code know that there are payloads queued for flush.
	 * This happens when AddCache() calls cause the payload size to exceed MaxPayloadSize. 
	 * Calling code needs to notice this and flush the queue.
	 */
	ANALYTICSET_API bool HasFlushesQueued() const;

	/**
	 * Gets the number of cached events (doesn't include any flushes that are already queued for flush). 
	 */
	ANALYTICSET_API int GetNumCachedEvents() const;

	/**
	 * Sets the preallocated payload size
	 * @param InPreallocatedCacheSize size to preallocate the payload buffer to during init and after flushing. If negative, will match the INI-configured MaximumPayloadSize.
	 */
	ANALYTICSET_API void SetPreallocatedPayloadSize(int32 InSetPreallocatedPayloadSize);

	/** Gets the preallocated size of the payload buffer. */
	ANALYTICSET_API int32 GetSetPreallocatedPayloadSize() const;

	friend class Lock;

	/** For when you need to take a lock across multiple API calls */
	class Lock
	{
	public:
		UE_NODISCARD_CTOR explicit Lock(FAnalyticsProviderETEventCache& EventCache)
			: ScopedLock(EventCache.CachedEventsCS) {}

	private:
		UE::TScopeLock<FTransactionallySafeCriticalSection> ScopedLock;
	};

	static ANALYTICSET_API void OnStartupModule();

private:
	ANALYTICSET_API void QueueFlush();

	ANALYTICSET_API TSharedRef<FAnalyticsProviderETEventCache> CloneForImmediateMode() const;

	/**
	* Analytics event entry to be cached
	*/
	struct FAnalyticsEventEntry
	{
		/** name of event */
		FString EventName;
		/** local time when event was triggered */
		FDateTime TimeStamp;
		/** byte offset into the payload stream that this DateOffset will be stored. Used when flushing the payload to set the proper DateOffset. */
		int32 DateOffsetByteOffset;
		/** Total charts used by the event. Mostly used for debugging large payloads. */
		int32 EventSizeChars;
		/**
		* Constructor. Requires rvalue-refs to ensure we move values efficiently into this struct.
		*/
		FAnalyticsEventEntry(FString&& InEventName, int32 InDateOffsetByteOffset, int32 InEventSizeChars)
			: EventName(MoveTemp(InEventName))
			, TimeStamp(FDateTime::UtcNow())
			, DateOffsetByteOffset(InDateOffsetByteOffset)
			, EventSizeChars(InEventSizeChars)
		{}
	};

	int32 MaximumPayloadSize;
	int32 PreallocatedPayloadSize;

	/**
	* List of analytic events pending a server update .
	* NOTE: The following members MUST be accessed inside a lock on CachedEventsCS!!
	*/
	TArray<FAnalyticsEventEntry> CachedEventEntries;
	TArray<uint8> CachedEventUTF8Stream;
	TArray<FAnalyticsEventAttribute> CachedDefaultAttributes;
	TArray<uint8> CachedDefaultAttributeUTF8Stream;
	TArray<TArray<uint8>> FlushQueue;

	/** Critical section for updating the CachedEvents. Mutable to allow const methods to access the list. */
	mutable FTransactionallySafeCriticalSection CachedEventsCS;
};
