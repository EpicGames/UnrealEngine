// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnalyticsEventAttribute.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Interfaces/IAnalyticsPropertyStore.h"
#include "Misc/Timespan.h"
#include "Templates/SharedPointer.h"

class IAnalyticsPropertyStore;
class IAnalyticsSessionSummarySender;
struct FAnalyticsEventAttribute;
template <typename T> class TAnalyticsProperty;

/** Defines how the principal process (the application for which analytics is gathered) exited. When the analytics session summary manager aggregates the summaries for a session,
    it looks for 'ShutdownTypeCode' key and if the key is found, it converts it to its string representation and add the 'ShutdownType' property known by the analytics backend. */
enum class EAnalyticsSessionShutdownType : int32
{
	Shutdown,
	Terminated,
	Debugged,
	Crashed,
	Abnormal,
	Unknown,
	Hang,
};

/**
 * Converts the specified shutdown code to its string representation.
 */
ANALYTICSET_API const TCHAR* LexToString(EAnalyticsSessionShutdownType ShutdownTypeCode);

/**
 * Manages the lifecycle of analytics summaries created by an application and its subsidiary processes. It manages the
 * summary files created by the processes, tracks which processes are working together, aggregates the summaries
 * from collaborating processes, discards outdated sessions and sends the final summary reports.
 *
 * Several processes can collaborate to collect analytics for one principal process. The common case being the Editor acting as
 * the principal process and CrashReportClient (CRC) acting as a subsidiary process. The principal process and its subsidiaries
 * form a group of processes tied together. The last process of the group to exit is responsible to aggregates and sends the final
 * report for the principal process. For any reasons, if the final report cannot be sent by a group, the responsibility to deal
 * with the left over is put on the principal process. The principal process is responsible to clean up the left over from its
 * previous execution(s) and possibly salvage and send reports delayed.
 */
class FAnalyticsSessionSummaryManager
{
public:
	/** Special key handled by the manager for the analytics backend. Its value must be one of EAnalyticsSessionShutdownType enum value. */
	static ANALYTICSET_API const TAnalyticsProperty<int32> ShutdownTypeCodeProperty;

	/** Special key handled in CRC to decide if an abnormal shutdown crash report should be generated. */
	static ANALYTICSET_API const TAnalyticsProperty<bool> IsUserLoggingOutProperty;

public:
	/**
	 * Constructs a manager for the principal process, usually the main application.
	 * @param ProcessName The process tag name of the principal process. Ex. "Editor".
	 * @param ProcessGroupId A unique ID shared by the principal and subsidiary processes. This is how information created by various processes is associated.
	 * @param UserId The current analytics user ID. @see IAnalyticsProviderET
	 * @param AppId The current application ID used by analytics to identify the application. @see IAnalyticsProviderET
	 * @param AppVersion the current application version used by the analytics for this application. @see IAnalyticsProviderET
	 * @param SessionId the current session ID assigned by the analytics. @see IAnalyticsProviderET
	 * @param SavedDir The directory where files will be persisted. Should be the same for the principal and the subsidiary processes. If unspecified, the manager uses its internal default.
	 */
	ANALYTICSET_API FAnalyticsSessionSummaryManager(const FString& ProcessName, const FString& ProcessGroupId, const FString& UserId, const FString& AppId, const FString& AppVersion, const FString& SessionId, const FString& SavedDir = TEXT(""));

	/**
	 * Constructs a manager for a subsidiary process, usually a companion process that collects extra information on the behalf of the principal process.
	 * @param ProcessName The process tag name of the subsidiary process. Ex. "CrashReportClient".
	 * @param ProcessGroupId A unique ID shared by the principal and subsidiary processes. This is how information created by various processes is associated.
	 * @param PrincipalProcessId The principal process ID to which this subsidiary process is associated.
	 * @param SavedDir The directory where files will be persisted. Should be the same for the principal and the subsidiary processes. If unspecified, the manager uses its internal default.
	 */
	ANALYTICSET_API FAnalyticsSessionSummaryManager(const FString& ProcessName, const FString& ProcessGroupId, uint32 PrincipalProcessId, const FString& SavedDir = TEXT(""));

	/**
	 * Destructor.
	 */
	ANALYTICSET_API ~FAnalyticsSessionSummaryManager();

	/**
	 * Discovers and salvages/sends/cleans up left over from previous execution(s), if any. Only the principal process can send left over sessions.
	 */
	ANALYTICSET_API void Tick();

	/**
	 * Creates a new property store associated to this manager process group. The manager only sends an analytics session summary when all
	 * collecting processes have closed their property store. Ensure to flush and release the store before calling the manager Shutdown() to
	 * ensure the session is sent as soon a possible. Subsequent calls to MakeStore will increment an internal counter in the store's filename.
	 * @param InitialCapacity The amount of space to reserve in the file.
	 */
	ANALYTICSET_API TSharedPtr<IAnalyticsPropertyStore> MakeStore(uint32 InitialCapacity);

	/**
	 * Sets a summary sender to enable the manager to send sessions. If the sender is null, the sessions are not sent. If the
	 * sender is set, the manager will periodically search and send pending sessions, usually the one left over from
	 * processes that died unexpectedly. As a rule, the principal process is responsible to send the left over summary sessions
	 * and a subsidiary process only sends the current sessions summary if it is the last process of the group to exit. The manager
	 * deletes the session data while sending.
	 */
	ANALYTICSET_API void SetSender(TSharedPtr<IAnalyticsSessionSummarySender> Sender);

	/**
	 * Sets a the user id used for reporting analytics. Allows for changing the application user after startup. Will set the user id on
	 * all existing stores created from this session summary manager.
	 */
	ANALYTICSET_API void SetUserId(const FString& UserId);

	/**
	 * Shuts down and sends the session summaries that can be sent, if a sender is set. Remember to flush and discard the
	 * property store returned by MakeStore() before shutting down to prevent delaying the report.
	 * @param bDiscard Whether the current session is discarded or not. Discarded sessions are not sent.
	 * @see SetSender()
	 * @see MakeStore()
	 */
	ANALYTICSET_API void Shutdown(bool bDiscard = false);

	/**
	 * Cleans up all expired files that match the analytics session filename pattern. The manager automaticallly clean up if
	 * it runs, but if the analytics is off and the manager is not instantiated anymore, some dead files can be left over.
	 * @param SavedDir The directory used to save the analytic files. If unspecified, use the internal default.
	 */
	static ANALYTICSET_API void CleanupExpiredFiles(const FString& SavedDir = TEXT(""));

	/**
	 * Returns the age at which a session is considered expired and shouldn't be sent anymore.
	 * @return The expiration delay.
	 */
	static FTimespan GetSessionExpirationAge() { return FTimespan::FromDays(30); }

private:
	/** Information about a property store used by a process. */
	struct FPropertyFileInfo
	{
		/** The ID of the process that wrote the property file. Ex. CRC can write a summary for an Editor session. */
		uint32 ProcessId = 0;

		/** The tag name of the process that wrote the property file. */
		FString ProcessName;

		/** Pathname to the property file. */
		FString Pathname;
	};

	/** Lists the files created by a process group. */
	struct FProcessGroup
	{
		/** Identify the process for which the summary was written. */
		uint32 PrincipalProcessId = 0;

		/** The principal process name. */
		FString PrincipalProcessName;

		/** The list of files generated by the processes in the group. */
		TArray<FPropertyFileInfo> PropertyFiles;
	};

private:
	/** Delegate constructor. */
	ANALYTICSET_API FAnalyticsSessionSummaryManager(const FString& ProcessName, const FString& ProcessGroupId, uint32 InCurrentProcessId, uint32 PrincipalProcessId, const FString& UserId, const FString& AppId, const FString& AppVersion, const FString& SessionId, const FString& SessionRootDir);

	/** Returns all processes group (key is process group id) for which files exists. */
	ANALYTICSET_API TMap<FString, FProcessGroup> GetSessionFiles() const;

	/** Processes the information collected by the principal and its subsidiary processes and decides if a summmary can be aggregated and sent. */
	ANALYTICSET_API void ProcessSummary(const FString& ProcessGroupId, const FProcessGroup& ProcessGroup);

	/** Loads the summaries created for the application and its subsidiary processes (like Editor/CRC combo) and aggregates summary properties. */
	ANALYTICSET_API bool AggregateSummaries(const FString& ProcessGroupId, const FProcessGroup& ProcessGroup, TArray<FAnalyticsEventAttribute>& OutSummaryProperties, TArray<FAnalyticsEventAttribute>& OutInternalProperties);

	/** Deletes the files used to record and perist the properties. */
	ANALYTICSET_API bool CleanupFiles(const TArray<FPropertyFileInfo>& PropertyFiles, bool bOnSuccess);

	/** Checks if this process is allowed to process and send orphan files. */
	bool IsOrphanGroupsOwner() const { return bOrphanGroupOwner; }

	/** Returns whether this manager instance represents the principal process of a group. */
	ANALYTICSET_API bool IsPrincipalProcess() const;

	/** Prune the list of property stores for invalid entries. */
	ANALYTICSET_API void PruneTrackedPropertyStores();

private:
	TArray<TWeakPtr<IAnalyticsPropertyStore>> WeakPropertyStores;
	TSharedPtr<IAnalyticsSessionSummarySender> SummarySender;
	FString ProcessName;
	FString ProcessGroupId;
	FString UserId;
	FString AppId;
	FString AppVersion;
	FString SessionId;
	FString SessionRootPath;
	uint32 CurrentProcessId;
	uint32 PrincipalProcessId;
	uint32 StoreCounter;
	double NextOrphanSessionCheckTimeSecs;
	bool bOrphanGroupOwner = false;
	bool bIsPrincipal = false;
};
