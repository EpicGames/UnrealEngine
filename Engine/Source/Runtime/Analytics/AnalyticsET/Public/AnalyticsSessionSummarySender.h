// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "IAnalyticsSessionSummarySender.h"
#include "Templates/Function.h"

class IAnalyticsProviderET;
struct FAnalyticsEventAttribute;

/**
 * Sends the analytics session summary to Epic Games analytics service.
 */
class FAnalyticsSessionSummarySender : public IAnalyticsSessionSummarySender
{
public:
	/**
	 * Construction a summary sender.
	 * @param Provider The analytics provider that will be used to emits the summary event.
	 * @param ShouldEmitFilterFunc A filter function invoked for each properties before sending. If the function is bound and returns false, the property will be filtered out. If unbound, all properties passed to SendSesionSummary() are emitted.
	 */
	ANALYTICSET_API FAnalyticsSessionSummarySender(IAnalyticsProviderET& Provider, TFunction<bool(const FAnalyticsEventAttribute&)> ShouldEmitFilterFunc = TFunction<bool(const FAnalyticsEventAttribute&)>());

	/**
	 * Emits the summary events for the specified session id on behalf of the specified user/app/appversion. The function filters the properties by invoking the functor specified at construction.
	 * @param UserId The user emitting the report.
	 * @param AppId The application for which the reports is emitted.
	 * @param AppVersion The application version for which the report is emitted.
	 * @param SessionId The session is for which the report is emitted.
	 * @param Properties The list of properties that makes up the summary event.
	 */
	ANALYTICSET_API virtual bool SendSessionSummary(const FString& UserId, const FString& AppId, const FString& AppVersion, const FString& SessionId, const TArray<FAnalyticsEventAttribute>& Properties) override;

private:
	IAnalyticsProviderET& AnalyticsProvider;
	TFunction<bool(const FAnalyticsEventAttribute&)> ShouldEmitPropFunc;
};
