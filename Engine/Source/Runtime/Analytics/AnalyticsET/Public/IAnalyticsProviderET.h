// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnalyticsET.h"
#include "AnalyticsEventAttribute.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "HAL/PlatformMisc.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "Misc/Guid.h"
#include "Templates/Function.h"
#include "Templates/UnrealTemplate.h"

class IHttpRequest;

/** Fired immediately prior to the HTTP request for an analytics event being processed by HTTP */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPreAnalyticsEventProcessed, TSharedRef<IHttpRequest, ESPMode::ThreadSafe>& /*HttpRequest*/);

struct FAnalyticEventQueuedInfo
{
	const EAnalyticsRecordEventMode Mode;
	const FString& EventName;
	const TArray<FAnalyticsEventAttribute>& Attributes;
};

/** 
*	Fired on every analytic event being queued
*   boolean value should only be left alone or set to false, which will cause the event to not be queued.
*   FAnalyticEventQueuedInfo is only valid during the broadcast, you must copy any data from it that you need to keep
*/
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnAnalyticsEventQueued, bool& /* bQueueEvent If this boolean is made false by any registered callback, this event will not be queued.  Do not set this value to true */, 
	const FAnalyticEventQueuedInfo& /* All the details on the event that is being queued */)

/** ET specific analytics provider instance. Exposes additional APIs to support Json-based events, default attributes, and allowing events to be disabled (generally via hotfixing). */
class IAnalyticsProviderET : public IAnalyticsProvider
{
public:
	using IAnalyticsProvider::StartSession;
	using IAnalyticsProvider::RecordEvent;

	////////////////////////////////////////////////////////////////////////////////////
	// IAnalyticsProvider overrides

	/**
	 * This class augments RecordEvent with a version that takes the EventName by rvalue reference to save a string copy. Implement the base version in terms of this one.
	 */
	virtual void RecordEvent(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes) override
	{
		return RecordEvent(CopyTemp(EventName), Attributes);
	}

	/**
	 * This class augments RecordEvent with a version that takes the EventName by rvalue reference to save a string copy. Implement the base version in terms of this one.
	 */
	virtual void RecordEvent(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes, EAnalyticsRecordEventMode Mode) override
	{
		return RecordEvent(CopyTemp(EventName), Attributes, Mode);
	}

	/**
	 * This class augments StartSession with a version that takes the SessionID instead of always generating it. Implement the base version in terms of this one.
	 */
	virtual bool StartSession(const TArray<FAnalyticsEventAttribute>& Attributes) override
	{
		FGuid SessionGUID;
		FPlatformMisc::CreateGuid(SessionGUID);
		return StartSession(SessionGUID.ToString(EGuidFormats::DigitsWithHyphensInBraces), Attributes);
	}

	// IAnalyticsProvider overrides
	////////////////////////////////////////////////////////////////////////////////////


	/**
	 * Special setter to set the AppID, something that is not normally allowed for third party analytics providers.
	 *
	 * @param AppID The new AppID to set
	 */
	virtual void SetAppID(FString&& AppID) = 0;

	/**
	 * Method to get the AppID (APIKey)
	 *
	 * @return the AppID (APIKey)
	 */
	const FString& GetAppID() const { return GetConfig().APIKeyET; }

	/**
	 * Sets the AppVersion.
	 *
	 * @param AppVersion The new AppVersion.
	 */
	virtual void SetAppVersion(FString&& AppVersion) = 0;

	/**
	* Method to get the AppVersion
	*
	* @return the AppVersion
	*/
	const FString& GetAppVersion() const { return GetConfig().AppVersionET; }

	/**
	 * Primary StartSession API. Allow move semantics to capture the attributes.
	 */
	virtual bool StartSession(FString InSessionID, const TArray<FAnalyticsEventAttribute>& Attributes) = 0;

	/**
	* Allows higher level code to abort logic to set up for a RecordEvent call by checking the filter that will be used to send the event first.
	*
	* @param EventName The name of the event.
	* @return true if the event will be recorded using the currently installed ShouldRecordEvent function
	*/
	virtual bool ShouldRecordEvent(const FString& EventName) const = 0;

	/**
	 * Primary RecordEvent API. Allow move semantics to capture the attributes.
	 */
	virtual void RecordEvent(FString&& EventName, const TArray<FAnalyticsEventAttribute>& Attributes) = 0;
	
	/**
	 * Overload to allow platforms to implement events that are immediately sent when supported. @see EAnalyticsRecordEventMode.
	 */
	virtual void RecordEvent(FString&& EventName, const TArray<FAnalyticsEventAttribute>& Attributes, EAnalyticsRecordEventMode Mode) = 0;

	/**
	 * Updates the default Domain and AltDomains.
	 */
	virtual void SetUrlDomain(const FString& Domain, const TArray<FString>& AltDomains) = 0;

	/**
	* Updates the Path
	*/
	virtual void SetUrlPath(const FString& Path) = 0;

	/**
	 * Sets a header to be included with analytics http requests
	 */
	virtual void SetHeader(const FString& HeaderName, const FString& HeaderValue) = 0;

	typedef TFunction<void(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attrs, bool bJson)> OnEventRecorded;

	/**
	* Set a callback to be invoked any time an event is queued.
	*
	* @param the callback
	*/
	virtual void SetEventCallback(const OnEventRecorded& Callback) = 0;

	/**
	* Blocks execution in the thread until all events have been flushed to the network.
	*/
	virtual void BlockUntilFlushed(float InTimeoutSec) = 0;

	/**
	 * Return the current provider configuration.
	 */
	virtual const FAnalyticsET::Config& GetConfig() const = 0;

	/** Callback used before any event is actually sent. Allows higher level code to disable events. */
	typedef TFunction<bool(const IAnalyticsProviderET& ThisProvider, const FString& EventName)> ShouldRecordEventFunction;

	/** Set an event filter to dynamically control whether an event should be sent. */
	virtual void SetShouldRecordEventFunc(const ShouldRecordEventFunction& ShouldRecordEventFunc) = 0;
	
	virtual FOnPreAnalyticsEventProcessed& OnPreAnalyticsEventProcessed() = 0;
	virtual FOnAnalyticsEventQueued& OnAnalyticsEventQueued() = 0;

private:
	/** Needed to support the old, unsafe GetDefaultAttributes() API. */
	mutable TArray<FAnalyticsEventAttribute> UnsafeDefaultAttributes;
};
