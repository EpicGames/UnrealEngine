// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnalyticsEventAttribute.h"

/**
 * Sends the analytics session summary to a backend service.
 */
class IAnalyticsSessionSummarySender
{
public:
	virtual ~IAnalyticsSessionSummarySender() = default;

	/**
	 * Emits the summary events for the specified session id on behalf of the specified user/app/appversion. The function filters the properties by invoking the functor specified at construction.
	 * @param UserId The user emitting the report.
	 * @param AppId The application for which the reports is emitted.
	 * @param AppVersion The application version for which the report is emitted.
	 * @param SessionId The session is for which the report is emitted.
	 * @param Properties The list of properties that makes up the summary event.
	 */
	virtual bool SendSessionSummary(const FString& UserId, const FString& AppId, const FString& AppVersion, const FString& SessionId, const TArray<FAnalyticsEventAttribute>& Properties) = 0;
};
