// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AnalyticsHorde : ModuleRules
	{
		public AnalyticsHorde(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
				"Core",
				"Analytics",
				"AnalyticsET"
				}
			);	
		}
	}
}
