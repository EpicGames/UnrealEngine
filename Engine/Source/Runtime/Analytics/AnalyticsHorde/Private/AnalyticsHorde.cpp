// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnalyticsHorde.h"
#include "AnalyticsET.h"
#include "IAnalyticsProviderET.h"
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"

IMPLEMENT_MODULE( FAnalyticsHorde, AnalyticsHorde );

void FAnalyticsHorde::StartupModule()
{
}

void FAnalyticsHorde::ShutdownModule()
{
}

FString GetServerURL()
{
	static FString ServerURL;

	if (ServerURL.IsEmpty())
	{
		if (false == FParse::Value(FCommandLine::Get(), TEXT("HordeServerUrl="), ServerURL))
		{
			ServerURL = FPlatformMisc::GetEnvironmentVariable(TEXT("UE_HORDE_URL"));
		}
	}
	return ServerURL;
}

FString GetTelemetryAPI()
{
	static FString TelemetryAPI;

	if (TelemetryAPI.IsEmpty())
	{
		if (false == FParse::Value(FCommandLine::Get(), TEXT("HordeTelemetryApi="), TelemetryAPI))
		{
			TelemetryAPI = FPlatformMisc::GetEnvironmentVariable(TEXT("UE_HORDE_TELEMETRY_API"));
		}
	}
	return TelemetryAPI;
}

TSharedPtr<IAnalyticsProvider> FAnalyticsHorde::CreateAnalyticsProvider(const FAnalyticsProviderConfigurationDelegate& GetConfigValue) const
{
	if (GetConfigValue.IsBound())
	{
		TSharedPtr<IAnalyticsProviderET> AnalyicsProviderET = FAnalyticsET::Get().CreateAnalyticsProviderET(GetConfigValue);

		if (AnalyicsProviderET.IsValid())
		{
			// Check if we have specified a Horde URL in the environment. This allows jobs running on Horde to send telemetry directly to the server they were run on
			const FString HordeServerURL = GetServerURL();

			if (!HordeServerURL.IsEmpty())
			{
				TArray<FString> AltDomains;
				AnalyicsProviderET->SetUrlDomain(HordeServerURL, AltDomains);
			}

			// Check if we have specified a Horde API to use
			const FString HordeTelemetryAPI = GetTelemetryAPI();

			if (!HordeTelemetryAPI.IsEmpty())
			{
			AnalyicsProviderET->SetUrlPath(HordeTelemetryAPI);
			}
		}

		return AnalyicsProviderET;
	}

	return nullptr;
}
