// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnalyticsProviderConfigurationDelegate.h"
#include "Interfaces/IAnalyticsProviderModule.h"
#include "Modules/ModuleManager.h"

class IAnalyticsProvider;

/**
 *  Public implementation of IAnalyticsProviderModule
 */
class FAnalyticsHorde : public IAnalyticsProviderModule
{
	//--------------------------------------------------------------------------
	// Module functionality
	//--------------------------------------------------------------------------
public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline FAnalyticsHorde& Get()
	{
		return FModuleManager::LoadModuleChecked< FAnalyticsHorde >( "AnalyticsHorde" );
	}

	//--------------------------------------------------------------------------
	// Configuration functionality
	//--------------------------------------------------------------------------
public:
	
	//--------------------------------------------------------------------------
	// provider factory functions
	//--------------------------------------------------------------------------
public:
	/**
	 * IAnalyticsProviderModule interface.
	 * Creates the analytics provider given a configuration delegate.
	 * The keys required exactly match the field names in the Config object. 
	 */
	ANALYTICSHORDE_API virtual TSharedPtr<IAnalyticsProvider> CreateAnalyticsProvider(const FAnalyticsProviderConfigurationDelegate& GetConfigValue) const override;

private:
	ANALYTICSHORDE_API virtual void StartupModule() override;
	ANALYTICSHORDE_API virtual void ShutdownModule() override;
};
