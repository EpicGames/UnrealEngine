// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AnalyticsLog : ModuleRules
	{
		public AnalyticsLog(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
						"Core"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
				"Analytics"
				}
			);
		}
	}
}
