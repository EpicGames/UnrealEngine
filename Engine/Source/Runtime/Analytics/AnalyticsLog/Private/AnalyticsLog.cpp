// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnalyticsLog.h"
#include "AnalyticsProviderLog.h"

IMPLEMENT_MODULE( FAnalyticsLog, AnalyticsLog );

void FAnalyticsLog::StartupModule()
{
}

void FAnalyticsLog::ShutdownModule()
{
}

TSharedPtr<IAnalyticsProvider> FAnalyticsLog::CreateAnalyticsProvider(const FAnalyticsProviderConfigurationDelegate& GetConfigValue) const
{
	if (GetConfigValue.IsBound())
	{
		return MakeShared<FAnalyticsProviderLog>(GetConfigValue);
	}

	return nullptr;
}

