// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnalyticsProviderLog.h"

#include "Analytics.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Tasks/Pipe.h"

FAnalyticsProviderLog::FAnalyticsProviderLog(const FAnalyticsProviderConfigurationDelegate& GetConfigValue)
{
	FString FileName = GetConfigValue.Execute(TEXT("FileName"), true);

	if (FileName.IsEmpty())
	{
		// Use default filename
		FileName = TEXT("Telemetry.json");
	}

	FString FolderPath = GetConfigValue.Execute(TEXT("FolderPath"), true);

	if (FolderPath.IsEmpty())
	{
		// See if there's a folder specified in the environment 
		FolderPath = FPlatformMisc::GetEnvironmentVariable(TEXT("UE_TELEMETRY_DIR"));

		if (FolderPath.IsEmpty())
		{
			// Use default output path
			FolderPath = FPaths::ProjectSavedDir() / TEXT("Telemetry");
		}
	}

	// Create the full output path
	if (FolderPath.IsEmpty() == false)
	{
		if (IFileManager::Get().MakeDirectory(*FolderPath, true))
		{
			WriterPipe = MakeUnique<UE::Tasks::FPipe>(TEXT("FAnalyticsProviderLog_Writer"));

			FString FilePath = FolderPath / FileName;
			FileWriter = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*FilePath, FILEWRITE_EvenIfReadOnly));
		}
	}
}

FAnalyticsProviderLog::~FAnalyticsProviderLog()
{
	if (WriterPipe.IsValid())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FAnalyticsProviderLog::EndSession::ClosePipe);
		WriterPipe->WaitUntilEmpty();
		WriterPipe.Reset();
	}
}

bool FAnalyticsProviderLog::SetSessionID(const FString& InSessionID)
{
	SessionID = InSessionID;
	return true;
}

FString FAnalyticsProviderLog::GetSessionID() const
{
	return SessionID;
}

void FAnalyticsProviderLog::SetUserID(const FString& InUserID)
{
	UserID = InUserID;

}

FString FAnalyticsProviderLog::GetUserID() const
{
	return UserID;
}

void FAnalyticsProviderLog::FlushEvents()
{
}

void FAnalyticsProviderLog::SetDefaultEventAttributes(TArray<FAnalyticsEventAttribute>&& Attributes)
{
	DefaultEventAttributes = Attributes;
}

TArray<FAnalyticsEventAttribute> FAnalyticsProviderLog::GetDefaultEventAttributesSafe() const
{
	return DefaultEventAttributes;
}

int32 FAnalyticsProviderLog::GetDefaultEventAttributeCount() const
{
	return DefaultEventAttributes.Num();
}

FAnalyticsEventAttribute FAnalyticsProviderLog::GetDefaultEventAttribute(int AttributeIndex) const
{
	return AttributeIndex < DefaultEventAttributes.Num() ? DefaultEventAttributes[AttributeIndex] : FAnalyticsEventAttribute();
}

bool FAnalyticsProviderLog::StartSession(const TArray<FAnalyticsEventAttribute>& Attributes)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAnalyticsProviderLog::StartSession);

	RecordEvent(TEXT("StartSession"), Attributes);

	return true;
}

void FAnalyticsProviderLog::EndSession()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAnalyticsProviderLog::EndSession);

	RecordEvent(TEXT("EndSession"));

	if (WriterPipe.IsValid())
	{
		WriterPipe->WaitUntilEmpty();
	}

	if (FileWriter.IsValid())
	{
		FileWriter->Flush();
		FileWriter->Close();

		FileWriter.Reset();
	}
}

void FAnalyticsProviderLog::RecordEvent(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes)
{	
	TRACE_CPUPROFILER_EVENT_SCOPE(FAnalyticsProviderLog::RecordEvent);

	static uint32 RecordId(0);

	if (FileWriter.IsValid() && WriterPipe.IsValid())
	{
		TStringBuilder<1024> Builder;

		// Log event as Newline - delimited JSON
		Builder.Appendf(TEXT("{\"EventName\":\"%s\""), *EventName);

		// Add the event timestamp field
		Builder.Appendf(TEXT(",\"TimestampUTC\":%f"), FDateTime::UtcNow().ToUnixTimestampDecimal());

		// Add the record Id and increment it
		Builder.Appendf(TEXT(",\"RecordId\":%d"), RecordId++);

		// Accumulate all the attributes together. We could have had two loops but this seems cleaner
		TArray<FAnalyticsEventAttribute> EventAttributes(Attributes);

		if (DefaultEventAttributes.Num())
		{
			// Append the defaults
			EventAttributes.Append(DefaultEventAttributes);
		}

		// Add all the attributes
		for (const FAnalyticsEventAttribute& Attribute : EventAttributes)
		{
			// This should be almost nearly true, but we should check and JSON'ify as needed
			if (Attribute.IsJsonFragment())
			{
				Builder.Appendf(TEXT(",\"%s\":%s"), *Attribute.GetName(), *Attribute.GetValue());
			}
			else
			{
				Builder.Appendf(TEXT(",\"%s\":\"%s\""), *Attribute.GetName(), *Attribute.GetValue());
			}
		}

		Builder.Append(TEXT("}"));

		FString FinalOutput = Builder.ToString();

		WriterPipe->Launch(TEXT("FAnalyticsProviderLog_WriteJob"), [this, FinalOutput = MoveTemp(FinalOutput)]()
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FAnalyticsProviderLog_WriteJob);
				
				FileWriter->Logf(TEXT("%s"), *FinalOutput);
				FileWriter->Flush();
			});
	}
}
