// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnalyticsProviderConfigurationDelegate.h"
#include "Interfaces/IAnalyticsProvider.h"

namespace UE::Tasks { class FPipe; }

/**
 * Implementation of the IAnalyticsProvider interface that exports telemetry events to a file as Newline - delimited JSON
 * By default,the log file is written to Saved/Telemetry folder of the application.
 * FileName and FolderPath can be overridden in the configuration
 * Here as a simple example of how a developer might implement and configure their own analytics provider for use with the StudioTelemerty plugin
 */
class FAnalyticsProviderLog : public IAnalyticsProvider
{
public:

	FAnalyticsProviderLog(const FAnalyticsProviderConfigurationDelegate& GetConfigValue);
	~FAnalyticsProviderLog();

	virtual bool StartSession(const TArray<FAnalyticsEventAttribute>& Attributes = {}) override;
	virtual void EndSession() override;

	virtual void FlushEvents() override;

	virtual void SetDefaultEventAttributes(TArray<FAnalyticsEventAttribute>&& Attributes)  override;
	virtual TArray<FAnalyticsEventAttribute> GetDefaultEventAttributesSafe() const override;
	virtual int32 GetDefaultEventAttributeCount() const  override;
	virtual FAnalyticsEventAttribute GetDefaultEventAttribute(int AttributeIndex) const  override;
	virtual bool SetSessionID(const FString& InSessionID) override;
	virtual void SetUserID(const FString& InUserID) override;

	virtual FString GetSessionID() const override;
	virtual FString GetUserID() const override;
	
	virtual void RecordEvent(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes = {}) override;

private:

	FString									UserID;
	FString									SessionID;
	TArray<FAnalyticsEventAttribute>		DefaultEventAttributes;	
	TUniquePtr<FArchive>					FileWriter;
	TUniquePtr<UE::Tasks::FPipe>			WriterPipe;
};
