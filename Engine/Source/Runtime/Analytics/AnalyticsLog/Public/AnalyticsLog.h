// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnalyticsProviderConfigurationDelegate.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Interfaces/IAnalyticsProviderModule.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

class IAnalyticsProvider;

/**
 *  Public implementation of IAnalyticsProviderModule that returns a JSON formatted FAnalayticsProviderLog object
 */
class FAnalyticsLog : public IAnalyticsProviderModule
{
	//--------------------------------------------------------------------------
	// Module functionality
	//--------------------------------------------------------------------------
public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline FAnalyticsLog& Get()
	{
		return FModuleManager::LoadModuleChecked< FAnalyticsLog >( "AnalyticsLog" );
	}

	/**
	 * IAnalyticsProviderModule interface.
	 */
	ANALYTICSLOG_API virtual TSharedPtr<IAnalyticsProvider> CreateAnalyticsProvider(const FAnalyticsProviderConfigurationDelegate& GetConfigValue) const override;

private:
	ANALYTICSLOG_API virtual void StartupModule() override;
	ANALYTICSLOG_API virtual void ShutdownModule() override;
};

