// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AnalyticsSwrve : ModuleRules
	{
		public AnalyticsSwrve(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.Add("Core");

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Analytics",
					"HTTP",
					"Json",
				}
			);

			if(Target.Platform != UnrealTargetPlatform.Win64 && !Target.IsInPlatformGroup(UnrealPlatformGroup.Unix) && Target.Platform != UnrealTargetPlatform.Mac)
			{
				PrecompileForTargets = PrecompileTargetsType.None;
			}
		}
	}
}
