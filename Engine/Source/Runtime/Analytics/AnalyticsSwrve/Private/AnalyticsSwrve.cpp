// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnalyticsSwrve.h"

#include "HAL/PlatformTime.h"
#include "Misc/Parse.h"
#include "Misc/CommandLine.h"
#include "Logging/LogScopedCategoryAndVerbosityOverride.h"
#include "Misc/ConfigCacheIni.h"
#include "Serialization/JsonSerializer.h"
#include "AnalyticsBuildType.h"
#include "Interfaces/IHttpResponse.h"
#include "HttpModule.h"
#include "Misc/EngineVersion.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "Analytics.h"

#if PLATFORM_DESKTOP

IMPLEMENT_MODULE( FAnalyticsSwrve, AnalyticsSwrve );

class FAnalyticsProviderSwrve : public IAnalyticsProvider
{
public:
	FAnalyticsProviderSwrve(const FAnalyticsSwrve::Config& ConfigValues);

	virtual bool StartSession(const TArray<FAnalyticsEventAttribute>& Attributes) override;
	virtual void EndSession() override;
	/** Swrve PC implementation doesn't cache events */
	virtual void FlushEvents() override {}

	virtual void SetUserID(const FString& InUserID) override;
	virtual FString GetUserID() const override;

	virtual FString GetSessionID() const override;
	virtual bool SetSessionID(const FString& InSessionID) override;

	virtual void SetDefaultEventAttributes(TArray<FAnalyticsEventAttribute>&& Attributes)  override;
	virtual TArray<FAnalyticsEventAttribute> GetDefaultEventAttributesSafe() const override;
	virtual int32 GetDefaultEventAttributeCount() const  override;
	virtual FAnalyticsEventAttribute GetDefaultEventAttribute(int AttributeIndex) const  override;

	virtual void RecordEvent(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes) override;
	virtual void RecordItemPurchase(const FString& ItemId, const FString& Currency, int PerItemCost, int ItemQuantity) override;
	virtual void RecordCurrencyPurchase(const FString& GameCurrencyType, int GameCurrencyAmount, const FString& RealCurrencyType, float RealMoneyCost, const FString& PaymentProvider) override;
	virtual void RecordCurrencyGiven(const FString& GameCurrencyType, int GameCurrencyAmount) override;

	virtual ~FAnalyticsProviderSwrve();

	FString GetAPIKey() const { return APIKey; }
private:
	/** Sends a request to SWRVE (helper func). */
	bool SendToSwrve(const FString& MethodName, const FString& OptionalParams, const FString& Payload);

	/** Sends a GET request to SWRVE (helper func). */
	bool SendToSwrve(const FString& MethodName, const FString& OptionalParams)
	{
		return SendToSwrve(MethodName, OptionalParams, FString());
	}
	/** Sends a GET request to SWRVE (helper func). */
	bool  SendToSwrve(const FString& MethodName)
	{
		return SendToSwrve(MethodName, FString(), FString());
	}

	bool bSessionInProgress;
	/** Swrve Game API Key - Get from your account manager */
	FString APIKey;
	/** Swrve API Server - should be http://api.swrve.com/ */
	FString APIServer;
	/** the unique UserID as passed to Swrve. */
	FString UserID;
	/** The AppVersion passed to Swrve. */
	FString AppVersion;
	
	// Variables used to determine if we are events at too fast a rate. Swrve imposes a soft-limit on events/sec, 
	// so this is only a suggestion that you might want to control your event rates.
	double NextEventRateDetectionWindowTimeSec;
	int EventRateDetectionWindowCount;
	static double kEventRateDetectionWindowTimeSec;
	static int kEventRateDetectionCountThreshold;

	TArray<FAnalyticsEventAttribute>		DefaultEventAttributes;

	/**
	 * Delegate called when an event Http request completes
	 */
	void EventRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded);

	/**
	 * Delegate called when an AB test Http request completes
	 */
	void ABTestRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded);
};

/** Swrve specifies that you shouldn't send more than 60 events/minute */
double FAnalyticsProviderSwrve::kEventRateDetectionWindowTimeSec = 60.0;
/** We'll check for 70 events / minute because it's really an aggregate total that matters, not really how many we send in a given minute. */
int FAnalyticsProviderSwrve::kEventRateDetectionCountThreshold = 70;

void FAnalyticsSwrve::StartupModule()
{
}

void FAnalyticsSwrve::ShutdownModule()
{
}

TSharedPtr<IAnalyticsProvider> FAnalyticsSwrve::CreateAnalyticsProvider(const FAnalyticsProviderConfigurationDelegate& GetConfigValue) const
{
	if (GetConfigValue.IsBound())
	{
		Config ConfigValues;
		ConfigValues.APIKeySwrve = GetConfigValue.Execute(Config::GetKeyNameForAPIKey(), true);
		ConfigValues.APIServerSwrve = GetConfigValue.Execute(Config::GetKeyNameForAPIServer(), false);
		ConfigValues.AppVersionSwrve = GetConfigValue.Execute(Config::GetKeyNameForAppVersion(), false);
		return CreateAnalyticsProvider(ConfigValues);
	}
	else
	{
		UE_LOGF(LogAnalytics, Warning, "CreateAnalyticsProvider called with an unbound delegate");
	}
	return NULL;
}

TSharedPtr<IAnalyticsProvider> FAnalyticsSwrve::CreateAnalyticsProvider(const Config& ConfigValues) const
{
	// If we didn't have a proper APIKey, return NULL
	if (ConfigValues.APIKeySwrve.IsEmpty())
	{
		UE_LOGF(LogAnalytics, Warning, "CreateAnalyticsProvider config did not contain required parameter %ls", *Config::GetKeyNameForAPIKey());
		return NULL;
	}
	return TSharedPtr<IAnalyticsProvider>(new FAnalyticsProviderSwrve(ConfigValues));
}

/**
 * After all the formality of downloading the AB test resources, this actually parses the payload and applies the diffs to the ConfigCache.
 */
static void ApplyABTestDiffs(const FString& ResourceDiffPayload)
{
	// we get back a list of tests, but it's a Json fragment. Turn it into a "real boy"
	FString ResourceDiffPayloadJson = FString("{\"Tests\":") + ResourceDiffPayload + TEXT("}");

	TSharedPtr<FJsonObject> JsonObject;
	TSharedRef< TJsonReader<> > Reader = TJsonReaderFactory<>::Create( ResourceDiffPayloadJson );

	if ( FJsonSerializer::Deserialize( Reader, JsonObject ) )
	{
		TArray<TSharedPtr<FJsonValue> > Tests = JsonObject->GetArrayField(TEXT("Tests"));
		for (TArray<TSharedPtr<FJsonValue> >::TConstIterator it(Tests); it; ++it)
		{
			TSharedPtr<FJsonObject> TestObj = (*it)->AsObject();
			FString IniKey = TestObj->GetStringField(TEXT("uid"));
			TArray<FString> IniTokens;
			IniKey.ParseIntoArray(IniTokens, TEXT("."), false);
			if (IniTokens.Num() < 2)
			{
				UE_LOGF(LogAnalytics, Warning, "Failed to parse resource name %ls into an INI file and section", *IniKey);
				continue;
			}
			TSharedPtr<FJsonObject> TestDiff = TestObj->GetObjectField(TEXT("diff"));
			for (TMap<FJsonObject::FStringType, TSharedPtr<FJsonValue>>::TConstIterator ResIt(TestDiff->Values); ResIt; ++ResIt)
			{
				FString NewValue = TestDiff->GetObjectField(ResIt.Key())->GetStringField(TEXT("new"));
				UE_LOGF(LogAnalytics, VeryVerbose, "Got an ABTtest resource for Engine[%ls]%ls=%ls", *IniTokens[1], *ResIt.Key(), *NewValue);
				if (IniTokens[0] == TEXT("Engine"))
				{
					GConfig->SetString(*IniTokens[1], *ResIt.Key(), *NewValue, GEngineIni);
				}
				else if (IniTokens[0] == TEXT("Game"))
				{
					GConfig->SetString(*IniTokens[1], *ResIt.Key(), *NewValue, GGameIni);
				}
				else
				{
					UE_LOGF(LogAnalytics, Warning, "Didn't understand INI resource name %ls for ABTest resource", *IniTokens[0]);
				}
			}
		}
	}
	else
	{
		UE_LOGF(LogAnalytics, Warning, "Failed to parse Swrve AB test resource diff payload '%ls'. Errors: %ls", *ResourceDiffPayload, *Reader->GetErrorMessage());
	}
}

/**
 * Perform any initialization.
 */
FAnalyticsProviderSwrve::FAnalyticsProviderSwrve(const FAnalyticsSwrve::Config& ConfigValues)
	:bSessionInProgress(false)
	,NextEventRateDetectionWindowTimeSec(FPlatformTime::Seconds() + kEventRateDetectionWindowTimeSec)
	,EventRateDetectionWindowCount(0)
{
	UE_LOGF(LogAnalytics, Verbose, "Initializing Swrve Analytics provider");

	APIKey = ConfigValues.APIKeySwrve;
	if (APIKey.IsEmpty())
	{
		UE_LOGF(LogAnalytics, Warning, "AnalyticsSwrve missing APIKey. No events will be processed.");
	}
	// allow the APIServer value to be empty and use defaults.
	APIServer = ConfigValues.APIServerSwrve.IsEmpty() 
		? GetAnalyticsBuildType() == EAnalyticsBuildType::Debug 
			? FAnalyticsSwrve::Config::GetDefaultAPIServerDebug()
			: FAnalyticsSwrve::Config::GetDefaultAPIServer()
		: ConfigValues.APIServerSwrve;

	// default to FEngineVersion::Current() if one is not provided, append FEngineVersion::Current() otherwise.
	AppVersion = ConfigValues.AppVersionSwrve.IsEmpty()
		? FString::Printf(TEXT("%d"), FEngineVersion::Current().GetChangelist())
		: FString::Printf(TEXT("%s.%d"), *ConfigValues.AppVersionSwrve, FEngineVersion::Current().GetChangelist());

	UE_LOGF(LogAnalytics, Log, "Swrve APIKey = %ls. APIServer = %ls. AppVersion = %ls", *APIKey, *APIServer, *AppVersion);

	// see if there is a cmdline supplied UserID.
#if !UE_BUILD_SHIPPING
	FString ConfigUserID;
	if (FParse::Value(FCommandLine::Get(), TEXT("ANALYTICSUSERID="), ConfigUserID, false))
	{
		SetUserID(ConfigUserID);
	}
#endif // !UE_BUILD_SHIPPING
}

FAnalyticsProviderSwrve::~FAnalyticsProviderSwrve()
{
	UE_LOGF(LogAnalytics, Verbose, "Destroying Swrve Analytics provider");
	EndSession();
}

/** Sends a request to SWRVE (helper func). */
bool FAnalyticsProviderSwrve::SendToSwrve(const FString& MethodName, const FString& OptionalParams, const FString& Payload)
{
	UE_LOGF(LogAnalytics, VeryVerbose, "Swrve Method: %ls. Params: %ls. Payload:\n%ls", *MethodName, *OptionalParams, *Payload);

	if (UserID.IsEmpty())
	{
		UE_LOGF(LogAnalytics, Warning, "%ls called without a valid UserID. Ignoring.", *MethodName);
		return false;
	}

	// check if we need to warn about sending too many events (don't both if we are suppressing the log!)
	UE_SUPPRESS(LogAnalytics, Warning, 
	{
		// increment the call count in this time window
		EventRateDetectionWindowCount++;
		double Now = FPlatformTime::Seconds();
		// if we're at or past time to check the rate (once per minute, usually)
		if (Now >= NextEventRateDetectionWindowTimeSec)
		{
			// Could be WAAAYYY past the time window, so compute accurate time passage since this window started.
			double TimePassedSec = Now - (NextEventRateDetectionWindowTimeSec - kEventRateDetectionWindowTimeSec);
			double ActualRate = EventRateDetectionWindowCount / TimePassedSec;
			double LimitRate = kEventRateDetectionCountThreshold / kEventRateDetectionWindowTimeSec;
			// if the actual rate exceeds the limit rate, then warn the user.
			if (ActualRate >= LimitRate)
			{
				UE_LOGF(LogAnalytics, Warning, "Sending too many events to Swrve (%d) in the past %.2lf seconds (Rate of %.2lf / sec, Max Rate is %.2lf / sec). You may be exceeding Swrve quotas.", EventRateDetectionWindowCount, TimePassedSec, ActualRate, LimitRate);
			}
			// reset teh time window.
			EventRateDetectionWindowCount = 0;
			NextEventRateDetectionWindowTimeSec = Now + kEventRateDetectionWindowTimeSec;
		}
	});

	// Create/send Http request for an event
	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();
	HttpRequest->SetHeader(TEXT("Content-Type"),
		Payload.IsEmpty() ? TEXT("text/plain") : TEXT("application/x-www-form-urlencoded; charset=utf-8"));
	HttpRequest->SetURL(FString::Printf(TEXT("%s%s?api_key=%s&user=%s&app_version=%s%s%s"),
		*APIServer, *MethodName, *APIKey, *UserID, *AppVersion, OptionalParams.IsEmpty() ? TEXT("") : TEXT("&"), *OptionalParams));
	HttpRequest->SetVerb(Payload.IsEmpty() ? TEXT("GET") : TEXT("POST"));
	HttpRequest->SetContentAsString(Payload);
	HttpRequest->OnProcessRequestComplete().BindRaw(this, &FAnalyticsProviderSwrve::EventRequestComplete);
	HttpRequest->ProcessRequest();

	return true;
}

/**
 * Start capturing stats for upload
 * Uses the unique ApiKey associated with your app
 */
bool FAnalyticsProviderSwrve::StartSession(const TArray<FAnalyticsEventAttribute>& Attributes)
{
	UE_LOGF(LogAnalytics, Log, "AnalyticsSwrve::StartSession [%ls]",*APIKey);

	bSessionInProgress = SendToSwrve(TEXT("1/session_start"));

	if (bSessionInProgress)
	{
		// send the session attributes
		RecordEvent(TEXT("SessionAttributes"), Attributes);

		// Create/send Http request to load the AB test resources
		TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();
		HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("text/plain"));
		HttpRequest->SetURL(FString::Printf(TEXT("https://abtest.swrve.com/api/1/user_resources_diff?api_key=%s&user=%s&app_version=%s"),
			*APIKey, *UserID, *AppVersion));
		HttpRequest->SetVerb(TEXT("GET"));
		HttpRequest->OnProcessRequestComplete().BindRaw(this, &FAnalyticsProviderSwrve::ABTestRequestComplete);
		HttpRequest->ProcessRequest();
	}

	return bSessionInProgress;
}

/**
 * End capturing stats and queue the upload 
 */
void FAnalyticsProviderSwrve::EndSession()
{
	if (bSessionInProgress)
	{
		SendToSwrve(TEXT("1/session_end"));
	}
	bSessionInProgress = false;
}

void FAnalyticsProviderSwrve::SetUserID(const FString& InUserID)
{
	// command-line specified user ID overrides all attempts to reset it.
	if (!FParse::Value(FCommandLine::Get(), TEXT("ANALYTICSUSERID="), UserID, false))
	{
		UE_LOGF(LogAnalytics, Log, "SetUserId %ls", *InUserID);
		UserID = InUserID;
	}
	else if (UserID != InUserID)
	{
		UE_LOGF(LogAnalytics, Log, "Overriding SetUserId %ls with cmdline UserId of %ls.", *InUserID, *UserID);
	}
}

FString FAnalyticsProviderSwrve::GetUserID() const
{
	return UserID;
}

FString FAnalyticsProviderSwrve::GetSessionID() const
{
	// Swrve doesn't support exposing the SessionID
	return FString();
}

bool FAnalyticsProviderSwrve::SetSessionID(const FString& InSessionID)
{
	// Swrve doesn't support exposing the SessionID
	return false;
}

/** Helper to log any swrve event. Used by all the LogXXX functions. */
void FAnalyticsProviderSwrve::RecordEvent(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes)
{
	// encode params as Json
	FString EventParams = "swrve_payload=";

	if (Attributes.Num() > 0)
	{
		EventParams += TEXT("{");

		// Add the default attributes
		for (int Ndx = 0; Ndx < DefaultEventAttributes.Num(); ++Ndx)
		{
			if (Ndx > 0)
			{
				EventParams += TEXT(",");
			}
			EventParams += FString("\"") + DefaultEventAttributes[Ndx].GetName() + TEXT("\": \"") + DefaultEventAttributes[Ndx].GetValue() + TEXT("\"");
		}

		// Add the event attributes
		for (int Ndx=0;Ndx<Attributes.Num();++Ndx)
		{
			if (Ndx > 0)
			{
				EventParams += TEXT(",");
			}
			EventParams += FString("\"") + Attributes[Ndx].GetName() + TEXT("\": \"") + Attributes[Ndx].GetValue()+ TEXT("\"");
		}
		EventParams += TEXT("}");
	}
	SendToSwrve(TEXT("1/event"), FString::Printf(TEXT("name=%s"), *EventName), EventParams);
}

void FAnalyticsProviderSwrve::RecordItemPurchase( const FString& ItemId, const FString& Currency, int PerItemCost, int ItemQuantity )
{
	SendToSwrve(TEXT("1/purchase"), FString::Printf(TEXT("item=%s&cost=%d&quantity=%d&currency=%s"), 
		*ItemId, PerItemCost, ItemQuantity, *Currency)); 
}

void FAnalyticsProviderSwrve::RecordCurrencyPurchase( const FString& GameCurrencyType, int GameCurrencyAmount, const FString& RealCurrencyType, float RealMoneyCost, const FString& PaymentProvider )
{
	SendToSwrve(TEXT("1/buy_in"), FString::Printf(TEXT("cost=%.2f&local_currency=%s&payment_provider=%s&reward_amount=%d&reward_currency=%s"), 
		RealMoneyCost, *RealCurrencyType, *PaymentProvider, GameCurrencyAmount, *GameCurrencyType)); 
}

void FAnalyticsProviderSwrve::RecordCurrencyGiven( const FString& GameCurrencyType, int GameCurrencyAmount )
{
	SendToSwrve(TEXT("1/currency_given"), FString::Printf(TEXT("given_currency=%s&given_amount=%d"), 
		*GameCurrencyType, GameCurrencyAmount)); 
}

void FAnalyticsProviderSwrve::EventRequestComplete(
	FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded)
{
	if (bSucceeded &&
		HttpResponse.IsValid())
	{
		UE_LOGF(LogAnalytics, VeryVerbose, "Swrve response for [%ls]. Code: %d. Payload: %ls", 
			*HttpRequest->GetURL(), HttpResponse->GetResponseCode(), *HttpResponse->GetContentAsString());
	}
	else
	{
		UE_LOGF(LogAnalytics, VeryVerbose, "Swrve response for [%ls]. No response", 
			*HttpRequest->GetURL());
	}
}

void FAnalyticsProviderSwrve::ABTestRequestComplete(
	FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded)
{
	if (bSucceeded &&
		HttpResponse.IsValid())
	{
		UE_LOGF(LogAnalytics, VeryVerbose, "Swrve ABTest response for [%ls]. Code: %d. Payload: %ls", 
			*HttpRequest->GetURL(), HttpResponse->GetResponseCode(), *HttpResponse->GetContentAsString());

		if (HttpResponse->GetResponseCode() == EHttpResponseCodes::Ok)
		{
			ApplyABTestDiffs(HttpResponse->GetContentAsString());
		}
		else
		{
			UE_LOGF(LogAnalytics, Warning, "Swrve returned failure for AB test resources request [%ls]. Code: %d. Payload: %ls", 
				*HttpRequest->GetURL(), HttpResponse->GetResponseCode(), *HttpResponse->GetContentAsString());
		}
	}
	else
	{
		UE_LOGF(LogAnalytics, Warning, "Swrve ABTest response for [%ls]. No response", 
			*HttpRequest->GetURL());
	}
}

void FAnalyticsProviderSwrve::SetDefaultEventAttributes(TArray<FAnalyticsEventAttribute>&& Attributes)
{
	DefaultEventAttributes = Attributes;
}

TArray<FAnalyticsEventAttribute> FAnalyticsProviderSwrve::GetDefaultEventAttributesSafe() const
{
	return DefaultEventAttributes;
}

int32 FAnalyticsProviderSwrve::GetDefaultEventAttributeCount() const
{
	return DefaultEventAttributes.Num();
}

FAnalyticsEventAttribute FAnalyticsProviderSwrve::GetDefaultEventAttribute(int AttributeIndex) const
{
	return DefaultEventAttributes[AttributeIndex];
}

#endif // PLATFORM_DESKTOP
