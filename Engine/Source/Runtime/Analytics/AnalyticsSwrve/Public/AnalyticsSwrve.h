// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnalyticsProviderConfigurationDelegate.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Interfaces/IAnalyticsProviderModule.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"

class IAnalyticsProvider;

/**
 * The public interface to this module
 */
class FAnalyticsSwrve : public IAnalyticsProviderModule
{
	//--------------------------------------------------------------------------
	// Module functionality
	//--------------------------------------------------------------------------
public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline FAnalyticsSwrve& Get()
	{
		return FModuleManager::LoadModuleChecked< FAnalyticsSwrve >( "AnalyticsSwrve" );
	}

	//--------------------------------------------------------------------------
	// Configuration functionality
	//--------------------------------------------------------------------------
public:
	/** Defines required configuration values for Swrve analytics provider. */
	struct Config
	{
		/** Swrve API Key - Get from your account manager */
		FString APIKeySwrve;
		/** Swrve API Server - Defaults if empty to GetDefaultAPIServer. FAnalytics::Debug build types default to GetDefaultAPIServerDebug. */
		FString APIServerSwrve;
		/** AppVersion - defines the app version passed to the provider. By default this will be FEngineVersion::Current().GetChangelist(). If you provide your own, ".<FEngineVersion::Current().GetChangelist()>" is appended to it. */
		FString AppVersionSwrve;

		/** KeyName required for APIKey configuration. */
		static FString GetKeyNameForAPIKey() { return TEXT("APIKeySwrve"); }
		/** KeyName required for APIServer configuration. */
		static FString GetKeyNameForAPIServer() { return TEXT("APIServerSwrve"); }
		/** KeyName required for AppVersion configuration. */
		static FString GetKeyNameForAppVersion() { return TEXT("AppVersionSwrve"); }
		/** Default value if no APIServer configuration is provided. */
		static FString GetDefaultAPIServer() { return TEXT("https://api.swrve.com/"); }
		/** Default value if BuildType == Debug if no APIServer configuration is provided. */
		static FString GetDefaultAPIServerDebug() { return TEXT("https://debug.api.swrve.com/"); }
	};

	//--------------------------------------------------------------------------
	// provider factory functions
	//--------------------------------------------------------------------------
public:
	/**
	 * IAnalyticsProviderModule interface.
	 * Creates the analytics provider given a configuration delegate.
	 * The keys required exactly match the field names in the Config object. 
	 */
	virtual TSharedPtr<IAnalyticsProvider> CreateAnalyticsProvider(const FAnalyticsProviderConfigurationDelegate& GetConfigValue) const;
	
	/** 
	 * Construct an analytics provider directly from a config object.
	 */
	virtual TSharedPtr<IAnalyticsProvider> CreateAnalyticsProvider(const Config& ConfigValues) const;

private:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

