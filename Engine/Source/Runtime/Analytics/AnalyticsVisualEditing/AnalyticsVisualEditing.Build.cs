// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AnalyticsVisualEditing : ModuleRules
	{
        public AnalyticsVisualEditing(ReadOnlyTargetRules Target) : base(Target)
		{
            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
					"Analytics",
    				"Core",
	    			"CoreUObject",
					"DeveloperSettings",
					"Engine"
				}
			);
		}
	}
}
