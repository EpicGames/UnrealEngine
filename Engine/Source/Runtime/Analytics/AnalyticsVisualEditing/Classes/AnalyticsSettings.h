// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"
#include "Engine/DeveloperSettings.h"
#include "AnalyticsSettings.generated.h"

struct FPropertyChangedEvent;

UCLASS(Abstract)
class ANALYTICSVISUALEDITING_API UAnalyticsSettingsBase
	: public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	FText SettingsDisplayName;

	UPROPERTY()
	FText SettingsTooltip;

	/** Helpers so that subclasses don't need to hardcode the strings in case they change */
	inline FString GetReleaseIniSection() const { return TEXT("Analytics"); }
	inline FString GetDebugIniSection() const { return TEXT("AnalyticsDebug"); }
	inline FString GetTestIniSection() const { return TEXT("AnalyticsTest"); }
	inline FString GetDevelopmentIniSection() const { return TEXT("AnalyticsDevelopment"); }
	inline FString GetIniName() const { return FString::Printf(TEXT("%sDefaultEngine.ini"), *FPaths::SourceConfigDir()); }

private:
/**
 * Because the analytics providers can be used outside of apps that include the UObject framework
 * we have to manually hook up the INI loading/saving which is done by these methods
 */
#if WITH_EDITOR
	virtual void PostInitProperties() override
	{
		Super::PostInitProperties();

		ReadConfigSettings();
	}

	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override
	{
		Super::PostEditChangeProperty(PropertyChangedEvent);

		WriteConfigSettings();

		GConfig->Flush(false, GetIniName());
	}
#endif

// UDeveloperSettings interface
	/** Gets the category for the settings, some high level grouping like, Editor, Engine, Game...etc. */
	virtual FName GetCategoryName() const override;

// UAnalyticsSettingsBase interface  - derived classes should implement
protected:
	/**
	 * Provides a mechanism to read the section based information into this UObject's properties
	 */
	virtual void ReadConfigSettings() {}
	/**
	 * Provides a mechanism to save this object's properties to the section based ini values
	 */
	virtual void WriteConfigSettings() {}

#if WITH_EDITOR
	/** Gets the section text for this  */
	virtual FText GetSectionText() const
	{
		return SettingsDisplayName;
	}
	/** Gets the description for the section, uses the classes ToolTip by default. */
	virtual FText GetSectionDescription() const
	{
		return SettingsTooltip;
	}
#endif
};

UCLASS()
class UAnalyticsSettings
	: public UAnalyticsSettingsBase
{
	GENERATED_UCLASS_BODY()

	/** The name of the plugin containing your desired analytics provider */
	UPROPERTY(EditAnywhere, DisplayName="Development", Category=Providers, meta=(ConfigRestartRequired=true))
	FString DevelopmentProviderName;

	/** The name of the plugin containing your desired analytics provider */
	UPROPERTY(EditAnywhere, DisplayName="Debug", Category=Providers, meta=(ConfigRestartRequired=true))
	FString DebugProviderName;

	/** The name of the plugin containing your desired analytics provider */
	UPROPERTY(EditAnywhere, DisplayName="Test", Category=Providers, meta=(ConfigRestartRequired=true))
	FString TestProviderName;

	/** The name of the plugin containing your desired analytics provider */
	UPROPERTY(EditAnywhere, DisplayName="Release", Category=Providers, meta=(ConfigRestartRequired=true))
	FString ReleaseProviderName;

// UAnalyticsSettingsBase interface
protected:
	/**
	 * Provides a mechanism to read the section based information into this UObject's properties
	 */
	virtual void ReadConfigSettings();
	/**
	 * Provides a mechanism to save this object's properties to the section based ini values
	 */
	virtual void WriteConfigSettings();
};
