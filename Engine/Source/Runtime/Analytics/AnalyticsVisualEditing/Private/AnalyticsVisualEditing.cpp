// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnalyticsVisualEditing.h"
#include "Modules/ModuleManager.h"
#include "Analytics.h"
#include "AnalyticsSettings.h"

IMPLEMENT_MODULE( FAnalyticsVisualEditingModule, AnalyticsVisualEditing );

static const FName AnalyticsName("Analytics");

#define LOCTEXT_NAMESPACE "Analytics"

UAnalyticsSettingsBase::UAnalyticsSettingsBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UAnalyticsSettings::UAnalyticsSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SettingsDisplayName = LOCTEXT("SettingsDisplayName", "Providers");
	SettingsTooltip = LOCTEXT("SettingsTooltip", "Configures which analytics provider to use per build type");
}

FName UAnalyticsSettingsBase::GetCategoryName() const
{
	return AnalyticsName;
}

void UAnalyticsSettings::ReadConfigSettings()
{
	ReleaseProviderName = FAnalytics::Get().GetConfigValueFromIni(GetIniName(), GetReleaseIniSection(), TEXT("ProviderModuleName"), true);
	DevelopmentProviderName = FAnalytics::Get().GetConfigValueFromIni(GetIniName(), GetDevelopmentIniSection(), TEXT("ProviderModuleName"), true);
	if (DevelopmentProviderName.Len() == 0)
	{
		DevelopmentProviderName = ReleaseProviderName;
	}
	TestProviderName = FAnalytics::Get().GetConfigValueFromIni(GetIniName(), GetTestIniSection(), TEXT("ProviderModuleName"), true);
	if (TestProviderName.Len() == 0)
	{
		TestProviderName = ReleaseProviderName;
	}
	DebugProviderName = FAnalytics::Get().GetConfigValueFromIni(GetIniName(), GetDebugIniSection(), TEXT("ProviderModuleName"), true);
	if (DebugProviderName.Len() == 0)
	{
		DebugProviderName = ReleaseProviderName;
	}
}

void UAnalyticsSettings::WriteConfigSettings()
{
	FAnalytics::Get().WriteConfigValueToIni(GetIniName(), GetReleaseIniSection(), TEXT("ProviderModuleName"), ReleaseProviderName);
	FAnalytics::Get().WriteConfigValueToIni(GetIniName(), GetDevelopmentIniSection(), TEXT("ProviderModuleName"), DevelopmentProviderName);
	FAnalytics::Get().WriteConfigValueToIni(GetIniName(), GetTestIniSection(), TEXT("ProviderModuleName"), TestProviderName);
	FAnalytics::Get().WriteConfigValueToIni(GetIniName(), GetDebugIniSection(), TEXT("ProviderModuleName"), DebugProviderName);
}

#undef LOCTEXT_NAMESPACE
