// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

/**
 * This module holds the base UObject class that allows visual editing of configuration settings in the Unreal Editor.
 * This is needed because the settings need to be usable outside of the UObject framework so these objects wrap
 * those settings for editor display purposes and write the non-UObject config property information out
 */
class FAnalyticsVisualEditingModule :
	public IModuleInterface
{
};

