// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "TelemetryRouter.h"
#include "TelemetryUtils.h"
#include "Misc/ScopeRWLock.h"

#if DO_CHECK
thread_local bool TelemetryRouterReentrancyGuard = false;
#endif

void FTelemetryRouter::CheckNotReentrant()
{
#if DO_CHECK
    check(TelemetryRouterReentrancyGuard == 0);
#endif
}

FTelemetryRouter::FTelemetryRouter()
{
}

FTelemetryRouter::~FTelemetryRouter()
{
}

FTelemetryRouter& FTelemetryRouter::Get()
{
    return FTelemetryUtils::GetRouter();
}

void FTelemetryRouter::ProvideTelemetryInternal(FGuid Key, FMemoryView Data) 
{
    CheckNotReentrant();
    FReadScopeLock Lock(SinkLock);
#if DO_CHECK
    TGuardValue Guard(TelemetryRouterReentrancyGuard, true);
#endif
    FSinkSet* Sinks = KeyToSinks.Find(Key);
    if (Sinks)
    {
        if (!ensureMsgf(Sinks->DataSize == Data.GetSize(), TEXT("Size mismatch mismatch with registered telemetry sink for guid %s. %" SIZE_T_X_FMT " vs %" SIZE_T_X_FMT), 
            *Key.ToString(), Sinks->DataSize, Data.GetSize() ))
        {
            return;
        }

        for (auto It = Sinks->Delegates.CreateIterator(); It; ++It)
        {
            bool bStillBound = It->Value(Data);
            if (!bStillBound)
            {
                It.RemoveCurrent();
            }
        }
    }
}

void FTelemetryRouter::RegisterTelemetrySinkInternal(FGuid Key, SIZE_T Size, FDelegateHandle InHandle, TFunction<bool(FMemoryView)> Sink)
{
    CheckNotReentrant();
    FWriteScopeLock Lock(SinkLock);
#if DO_CHECK
    TGuardValue Guard(TelemetryRouterReentrancyGuard, true);
#endif
    FTelemetryRouter::FSinkSet* Sinks = KeyToSinks.Find(Key);
    if (!Sinks)
    {
        Sinks = &KeyToSinks.Add(Key, FSinkSet(Size));
    }
    if (!ensureMsgf(Sinks->DataSize == Size, TEXT("Size mismatch mismatch with registered telemetry sink for guid %s. %" SIZE_T_X_FMT " vs %" SIZE_T_X_FMT), 
        *Key.ToString(), Sinks->DataSize, Size))
    {
        return;
    }
    Sinks->Delegates.Add(InHandle, MoveTemp(Sink));
}

void FTelemetryRouter::UnregisterTelemetrySinkInternal(FGuid Key, FDelegateHandle InHandle)
{
    CheckNotReentrant();
    FWriteScopeLock Lock(SinkLock);
#if DO_CHECK
    TGuardValue Guard(TelemetryRouterReentrancyGuard, true);
#endif
    FTelemetryRouter::FSinkSet* Sinks = KeyToSinks.Find(Key);
    if (Sinks)
    {
        Sinks->Delegates.Remove(InHandle);
    }
}