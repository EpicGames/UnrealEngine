// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "TelemetryUtils.h"
#include "TelemetryRouter.h"

IMPLEMENT_MODULE( FTelemetryUtils, TelemetryUtils );

FTelemetryUtils::FTelemetryUtils()
{
}

FTelemetryUtils::~FTelemetryUtils()
{
}

void FTelemetryUtils::StartupModule()
{
    Router = MakeUnique<FTelemetryRouter>();
}

void FTelemetryUtils::ShutdownModule()
{
    Router.Reset();
}
