// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "HAL/CriticalSection.h"
#include "Templates/Models.h"
#include "Memory/MemoryView.h"
#include "Misc/Guid.h"

#include <type_traits>

namespace UE::Telemetry::Private
{
    template<typename DATA_TYPE, typename = void>
    constexpr bool HasTelemetryID = false;

    template<typename DATA_TYPE>
    constexpr bool HasTelemetryID<
        DATA_TYPE,
        std::enable_if_t<std::is_same_v<std::decay_t<decltype(DATA_TYPE::TelemetryID)>, FGuid>>> = true;
    
    template <typename DATA_TYPE>
    auto GetDataKeyInternal()
    {
        if constexpr (HasTelemetryID<DATA_TYPE>)
        {
            return DATA_TYPE::TelemetryID;
        }
		else
		{
			return FGuid{};
		}
    }

    template <typename DATA_TYPE>
    FGuid GetDataKey() 
    {
        static_assert(HasTelemetryID<DATA_TYPE>, 
            "Unsupported type for telemetry router, must have a static constexpr FGuid member TelemetryID");
        // Silence duplicate compiler error that's less helpful than the static assert
        if constexpr (HasTelemetryID<DATA_TYPE>)
        {
            return DATA_TYPE::TelemetryID;
        }
        else
        {
            return FGuid{};
        }
    }
}

/** 
 * Provides an interface for routing structured telemetry data between prooducers (engine/editor systems)
 * and consumers (modules which will format the data for a particular telemetry endpoint)
 * 
 * Usage: 
 *  Declare a unique type for telemetry data. This type needs a static FGuid data member called TelemetryID.
 *  Provide data to any interested aggregators with the method ProvideTelemetry.
 *  Aggregators can register a callback for telemetry data with the method OnTelemetry.
 * 
 * The guid associated with a type for routing should not be relied on by consumers to be stable across engine versions 
 * or even across different processes, it is only used for runtime routing purposes. It should not be sent to telemetry endpoints.
 * 
 * @thread_safety Functions are internally synchronized. Re-entrant calls to the public API during callbacks are unsupported.
 *  Data is routed on the same thread it is provided. Users should copy data and schedule tasks to execute on a specified thread 
 *  if they require that. 
 */
class FTelemetryRouter 
{
public:
    FTelemetryRouter();
    ~FTelemetryRouter();
    
    static TELEMETRYUTILS_API FTelemetryRouter& Get();

    /**
     * Sends data in a type-safe manner to consumers expecting this data type. 
     * 
     * @param Data Strongly typed data to be sent. Type must implement a static method called GetTelemetryID returning an FGuid.
     */
    template<typename DATA_TYPE>
    inline void ProvideTelemetry(const DATA_TYPE& Data)
    {
        ProvideTelemetryInternal(UE::Telemetry::Private::GetDataKey<DATA_TYPE>(), MakeMemoryView(&Data, sizeof(DATA_TYPE)));
    }
    
    /** 
     * Registers a delegate as callback to receive telemetry of a certain type. 
     *
     * @return a handle that can be used to unregister this sink later so it is no longer called.
     */
    template<typename DATA_TYPE>
    inline FDelegateHandle OnTelemetry(TDelegate<void(const DATA_TYPE&)> Sink)
    {
        check(Sink.IsBound());
        FDelegateHandle Handle = Sink.GetHandle();
        RegisterTelemetrySinkInternal(UE::Telemetry::Private::GetDataKey<DATA_TYPE>(), sizeof(DATA_TYPE), Handle,
            [Sink=MoveTemp(Sink)](FMemoryView Data) -> bool {
                return Sink.ExecuteIfBound(*reinterpret_cast<const DATA_TYPE*>(Data.GetData()));
            });
        return Handle;
    }

    /** 
     * Registers a callable object as a callback receive telemetry of a certain type. 
     *
     * @return a handle that can be used to unregister this sink later so it is no longer called.
     */
    template<typename DATA_TYPE, typename CALLABLE,
        typename = std::enable_if_t<std::is_invocable_v<CALLABLE, const DATA_TYPE&>>
    >
    inline FDelegateHandle OnTelemetry(CALLABLE&& Sink)
    {
        FDelegateHandle Handle{ FDelegateHandle::GenerateNewHandle };
        RegisterTelemetrySinkInternal(UE::Telemetry::Private::GetDataKey<DATA_TYPE>(), sizeof(DATA_TYPE), Handle,
            [Sink=MoveTemp(Sink)](FMemoryView Data) -> bool {
                Sink(*reinterpret_cast<const DATA_TYPE*>(Data.GetData()));
                return true;
            });
        return Handle;
    }
    
    /** 
     * Removes a previously registered callback.
     */
    template<typename DATA_TYPE>
    inline void UnregisterTelemetrySink(FDelegateHandle Handle)
    {
        UnregisterTelemetrySinkInternal(UE::Telemetry::Private::GetDataKey<DATA_TYPE>(), Handle);
    }
    
private:
    /** 
     * Register a callback to receive telemetry. The callback should return false if the sink is stale and should be removed from future consideration.
     */
    TELEMETRYUTILS_API void RegisterTelemetrySinkInternal(FGuid Key, SIZE_T ExpectedSize, FDelegateHandle InHandle, TFunction<bool(FMemoryView)> Sink);
    TELEMETRYUTILS_API void UnregisterTelemetrySinkInternal(FGuid Key, FDelegateHandle InHandle);
    TELEMETRYUTILS_API void ProvideTelemetryInternal(FGuid Key, FMemoryView Data);
    TELEMETRYUTILS_API void CheckNotReentrant();
    
    FRWLock SinkLock;
    
    struct FSinkSet
    {
        SIZE_T DataSize;
        TMap<FDelegateHandle, TFunction<bool(FMemoryView)>> Delegates;
        
        FSinkSet(SIZE_T InDataSize)
            : DataSize(InDataSize)
        {
        }
    };

    TMap<FGuid, FSinkSet> KeyToSinks;
};