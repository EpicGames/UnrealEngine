// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Module containing utility functions for providing telemetry on the engine and games.
// Does not integrate with any specific telemetry endpoint.
// Provides functions for routing data from runtime/editor systems to telemetry sinks in a structured manner.

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class FTelemetryRouter;

class FTelemetryUtils : public IModuleInterface
{
public:
    FTelemetryUtils();
    virtual ~FTelemetryUtils();

    static inline FTelemetryUtils& Get()
    {
        return FModuleManager::LoadModuleChecked<FTelemetryUtils>("TelemetryUtils");
    }
    
    static inline bool IsAvailable()
    {
        return FModuleManager::Get().IsModuleLoaded("TelemetryUtils");
    }
    
    /** 
     * Returns an object used for routing structured telemetry data between producers and consumers.
     */
    static FTelemetryRouter& GetRouter()
    {
        return *FTelemetryUtils::Get().Router.Get();
    }

private:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
    
private:
    TUniquePtr<FTelemetryRouter> Router;
};
