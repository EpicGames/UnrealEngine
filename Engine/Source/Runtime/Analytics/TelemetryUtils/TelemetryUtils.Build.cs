// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class TelemetryUtils : ModuleRules
	{
		public TelemetryUtils(ReadOnlyTargetRules Target) : base(Target)
		{			
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
				}
				);
			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
			CppStandard = CppStandardVersion.Cpp20;
		}
	}
}
