// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "TelemetryRouter.h"
#include "TestHarness.h"

struct FTestTelemetryData
{
    int32 Count = 0;

    static constexpr inline FGuid TelemetryID = FGuid(0x4830c7cf, 0x2c184669, 0x89b299c2, 0x3134f2b3);
};

TEST_CASE("TelemetryUtils::Router::Single", "[Smoke]")
{
    FTelemetryRouter Router;

    int32 Increment = 4;
    FTestTelemetryData DataReceived;
    FDelegateHandle Handle = Router.OnTelemetry<FTestTelemetryData>(
        [&DataReceived](const FTestTelemetryData& In) {
            DataReceived.Count += In.Count;
        });
    
    Router.ProvideTelemetry<FTestTelemetryData>(FTestTelemetryData{ 
        .Count = Increment
    });
    CHECK_EQUAL(DataReceived.Count, Increment);
    Router.UnregisterTelemetrySink<FTestTelemetryData>(Handle);
    Router.ProvideTelemetry<FTestTelemetryData>(FTestTelemetryData{
        .Count = Increment
    });
    CHECK_EQUAL(DataReceived.Count, Increment);
}

TEST_CASE("TelemetryUtils::Router::Multiple", "[Smoke]")
{
    FTelemetryRouter Router;

    FTestTelemetryData ReceivedA;
    FTestTelemetryData ReceivedB;
    
    FDelegateHandle HandleA = Router.OnTelemetry<FTestTelemetryData>(
        [&ReceivedA](const FTestTelemetryData& In) {
            ReceivedA.Count += In.Count;
        });
    FDelegateHandle HandleB = Router.OnTelemetry<FTestTelemetryData>(
        [&ReceivedB](const FTestTelemetryData& In) {
            ReceivedB.Count += In.Count;
        });
    
    Router.ProvideTelemetry<FTestTelemetryData>(FTestTelemetryData{ 
        .Count = 1
    });
    CHECK_EQUAL(ReceivedA.Count, 1);
    CHECK_EQUAL(ReceivedB.Count, 1);
    Router.UnregisterTelemetrySink<FTestTelemetryData>(HandleA);
    Router.ProvideTelemetry<FTestTelemetryData>(FTestTelemetryData{
        .Count = 1
    });
    CHECK_EQUAL(ReceivedA.Count, 1);
    CHECK_EQUAL(ReceivedB.Count, 2);
}

TEST_CASE("TelemetryUtils::Router::WeakBinding", "[Smoke]")
{
    FTelemetryRouter Router;
    
    struct FCallee
    {
        FCallee()
        {
        }
        ~FCallee()
        {
        }
        
        void ReceiveTelemetry(const FTestTelemetryData& In)
        {
            ReceivedData.Count += In.Count;
        }
        
        FTestTelemetryData ReceivedData;
    };

    int32 Increment = 4;
    TWeakPtr<FCallee> WeakCallee;
    {
        TSharedRef<FCallee> Callee = MakeShared<FCallee>();
        FDelegateHandle Handle = Router.OnTelemetry<FTestTelemetryData>(
            TDelegate<void(const FTestTelemetryData&)>::CreateSP(Callee, &FCallee::ReceiveTelemetry)
        );
        
        Router.ProvideTelemetry<FTestTelemetryData>(FTestTelemetryData{ 
            .Count = Increment
        });
        CHECK_EQUAL(Callee->ReceivedData.Count, Increment);
        
        // Delete object without unregistering data sink
        WeakCallee = Callee;
        CHECK(WeakCallee.Pin().IsValid());
    }

    Router.ProvideTelemetry<FTestTelemetryData>(FTestTelemetryData{
        .Count = Increment
    });
    CHECK(!WeakCallee.Pin().IsValid());
}

#endif